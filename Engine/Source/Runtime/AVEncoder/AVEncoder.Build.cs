// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;
using System;

[SupportedPlatforms("Linux")]
[SupportedPlatformGroups("Windows")]
public class AVEncoder : ModuleRules
{
	public AVEncoder(ReadOnlyTargetRules Target) : base(Target)
	{
		// Without these two compilation fails on VS2017 with D8049: command line is too long to fit in debug record.
		bLegacyPublicIncludePaths = false;
		DefaultBuildSettings = BuildSettingsVersion.Latest;

		// PCHUsage = PCHUsageMode.NoPCHs;

		// PrecompileForTargets = PrecompileTargetsType.None;

		PrivateDependencyModuleNames.AddRange(new string[] {
			"Engine",
			"SignalProcessing"
		});

		PublicDependencyModuleNames.AddRange(new string[] {
			"RenderCore",
			"Core",
			"RHI",
			"CUDA"
			// ... add other public dependencies that you statically link with here ...
		});

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows) || Target.IsInPlatformGroup(UnrealPlatformGroup.Unix))
		{
			PublicIncludePathModuleNames.Add("VulkanRHI");
		}

		if (Target.IsInPlatformGroup(UnrealPlatformGroup.Windows))
		{
			PublicIncludePathModuleNames.AddRange([
				"D3D11RHI",
				"D3D12RHI"
			]);

			AddEngineThirdPartyPrivateStaticDependencies(Target, "DX11", "DX12");

			PublicSystemLibraries.AddRange(new string[] {
				"mfplat.lib",
				"mfuuid.lib",
			});

			PublicDelayLoadDLLs.Add("Mfreadwrite.dll");
		}
	}
}
