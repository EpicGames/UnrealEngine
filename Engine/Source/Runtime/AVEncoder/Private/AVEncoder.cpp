// Copyright Epic Games, Inc. All Rights Reserved.
#include "AVEncoder.h"

DEFINE_LOG_CATEGORY(LogAVEncoder);
CSV_DECLARE_CATEGORY_EXTERN(AVEncoder);
CSV_DEFINE_CATEGORY(AVEncoder, true);
