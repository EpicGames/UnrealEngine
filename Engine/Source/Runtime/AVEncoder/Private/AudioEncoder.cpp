// Copyright Epic Games, Inc. All Rights Reserved

#include "AudioEncoder.h"
#include "Misc/ScopeLock.h"

namespace AVEncoder
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FAudioEncoder::RegisterListener(IAudioEncoderListener& Listener)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FScopeLock lock{ &ListenersMutex };
		check(Listeners.Find(&Listener) == INDEX_NONE);
		Listeners.AddUnique(&Listener);
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FAudioEncoder::UnregisterListener(IAudioEncoderListener& Listener)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FScopeLock lock{ &ListenersMutex };
		int32 Count = Listeners.Remove(&Listener);
		check(Count == 1);
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FAudioEncoder::OnEncodedAudioFrame(const FMediaPacket& Packet)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FScopeLock lock{ &ListenersMutex };
		for (auto&& L : Listeners)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			L->OnEncodedAudioFrame(Packet);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		
	}
}
