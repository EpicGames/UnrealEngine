// Copyright Epic Games, Inc. All Rights Reserved.

#include "AudioEncoderFactory.h"
#include "AVEncoder.h"

#if PLATFORM_WINDOWS
#include "Encoders/WmfAudioEncoder.h"
#endif

namespace AVEncoder
{
	bool GDefaultFactoriesRegistered = false;
	void RegisterDefaultFactories();

	void DoDefaultRegistration()
	{
		if (GDefaultFactoriesRegistered)
			return;
		RegisterDefaultFactories();
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TArray<FAudioEncoderFactory*> FAudioEncoderFactory::Factories;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FAudioEncoderFactory::RegisterFactory(FAudioEncoderFactory& Factory)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		DoDefaultRegistration();

		Factories.AddUnique(&Factory);
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FAudioEncoderFactory::UnregisterFactory(FAudioEncoderFactory& Factory)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		Factories.Remove(&Factory);
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAudioEncoderFactory* FAudioEncoderFactory::FindFactory(const FString& Codec)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		DoDefaultRegistration();

		for (auto&& Factory : Factories)
		{
			if (Factory->GetSupportedCodecs().Find(Codec) != INDEX_NONE)
			{
				return Factory;
			}
		}

		return nullptr;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FAudioEncoderFactory*> FAudioEncoderFactory::GetAllFactories()
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		DoDefaultRegistration();

		return Factories;
	}

	void RegisterDefaultFactories()
	{
		// We need to set this at the top, otherwise RegisterFactory will call this recursively
		GDefaultFactoriesRegistered = true;

#if PLATFORM_WINDOWS
		// Generic Windows/XBox Wmf encoder
		static FWmfAudioEncoderFactory WmfAudioEncoderFactory;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FAudioEncoderFactory::RegisterFactory(WmfAudioEncoderFactory);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif


		// Log all available encoders
		{
			auto CodecsInfo = [&](auto&& Factories) -> FString
			{
				FString Str;
				for (auto&& Factory : Factories)
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					Str += FString::Printf(TEXT(", %s(%s) "), Factory->GetName(), *FString::Join(Factory->GetSupportedCodecs(), TEXT("/")));
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}
				
				if (Str.IsEmpty())
				{
					return FString("None");
				}
				else
				{
					return Str;
				}
			};

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			UE_LOGF(LogAVEncoder, Log, "Available audio encoders: %ls ", *CodecsInfo(FAudioEncoderFactory::GetAllFactories()));
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		
	}
}