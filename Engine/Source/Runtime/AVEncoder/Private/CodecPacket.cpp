// Copyright Epic Games, Inc. All Rights Reserved.

#include "CodecPacket.h"
#include "CoreMinimal.h"

namespace
{
	struct FDataCopyDeleter
	{
		void operator()(uint8* p)
		{
			FMemory::Free(p);
		}
	};

	void DataCopyDeleter(uint8* p)
	{
		FMemory::Free(p);
	}
}

namespace AVEncoder
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FCodecPacket FCodecPacket::Create(const uint8* InData, uint32 InDataSize)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		uint8* DataCopy = static_cast<uint8*>(FMemory::Malloc(InDataSize));
		FMemory::BigBlockMemcpy(DataCopy, InData, InDataSize);

		TSharedPtr<uint8> Data(DataCopy, [](uint8* Obj){ DataCopyDeleter(Obj); });

		FCodecPacket Packet;
		Packet.Data = Data;
		Packet.DataSize = InDataSize;
		return Packet;
	}
} /* namespace AVEncoder */
