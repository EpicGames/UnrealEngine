// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoder.h"
#include "VideoDecoderCommon.h"
#include "VideoDecoderAllocationTypes.h"

DEFINE_LOG_CATEGORY(LogVideoDecoder);

namespace AVEncoder
{

struct FVideoDecoder::FPlatformDecoderAllocInterface
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderMethodsWindows	AllocMethods = {};
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	void*						ClientData = nullptr;
};


FVideoDecoder::~FVideoDecoder()
{
	// This must have been released via ReleaseDecoderAllocationInterface() !
	check(PlatformDecoderAllocInterface == nullptr);
}

bool FVideoDecoder::CreateDecoderAllocationInterface()
{
	if (!PlatformDecoderAllocInterface)
	{
		check(PlatformDecoderAllocInterface == nullptr);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FPlatformDecoderAllocInterface* Interface = new FPlatformDecoderAllocInterface {};
		// TODO: Platform specific!!
		Interface->AllocMethods.MagicCookie = 0x57696e58;		// 'WinX'
		Interface->AllocMethods.This = nullptr;					// This will become the client's 'this' pointer
		Interface->AllocMethods.GetD3DDevice = nullptr;			// Method to get the current D3D device from the client.
		Interface->AllocMethods.AllocateFrameBuffer = nullptr;	// Method to allocate frame buffer memory from the client.
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Call into the client to set up the allocation interface.
		CreateDecoderAllocationInterfaceFN(&Interface->AllocMethods, &Interface->ClientData);

		// Check for success by testing the returned client data pointer.
		if (Interface->ClientData)
		{
			PlatformDecoderAllocInterface = Interface;
			return true;
		}
		else
		{
			delete Interface;
			return false;
		}
	}
	return true;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FVideoDecoder::ReleaseDecoderAllocationInterface()
{
	if (PlatformDecoderAllocInterface)
	{
		void* NoReturnValue = nullptr;
		ReleaseDecoderAllocationInterfaceFN(PlatformDecoderAllocInterface->ClientData, &NoReturnValue);
		delete PlatformDecoderAllocInterface;
		PlatformDecoderAllocInterface = nullptr;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void* FVideoDecoder::GetAllocationInterfaceMethods()
{
	return PlatformDecoderAllocInterface ? &PlatformDecoderAllocInterface->AllocMethods : nullptr;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
EFrameBufferAllocReturn FVideoDecoder::AllocateOutputFrameBuffer(FVideoDecoderAllocFrameBufferResult* OutBuffer, const FVideoDecoderAllocFrameBufferParams* InAllocParams)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	check(InAllocParams);
	check(OutBuffer);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (PlatformDecoderAllocInterface && PlatformDecoderAllocInterface->AllocMethods.AllocateFrameBuffer.IsBound())
	{
		return PlatformDecoderAllocInterface->AllocMethods.AllocateFrameBuffer.Execute(PlatformDecoderAllocInterface->AllocMethods.This, InAllocParams, OutBuffer);
	}
	return EFrameBufferAllocReturn::CODEC_Failure;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


} /* namespace AVEncoder */
