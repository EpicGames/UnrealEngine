// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoderH264_Dummy.h"

#ifdef AVENCODER_VIDEO_DECODER_AVAILABLE_H264_DUMMY


namespace AVEncoder
{

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FVideoDecoderOutputDummy : public FVideoDecoderOutput
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderOutputDummy(int32 w, int32 h, int64 pts)
	: Width(w), Height(h), PTS(pts)
	{}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FVideoDecoderOutputDummy()
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual int32 AddRef() override
	{
		return FPlatformAtomics::InterlockedIncrement(&RefCount);
	}

	virtual int32 Release() override
	{
		int32 c = FPlatformAtomics::InterlockedDecrement(&RefCount);
		// We do not release the allocated buffer from the application here.
		// This is meant to release only what the decoder uses internally, but not the
		// external buffers the application is working with!
		if (c == 0)
		{
			delete this;
		}
		return c;
	}

	virtual int32 GetWidth() const override
	{
		return Width;
	}
	virtual int32 GetHeight() const override
	{
		return Height;
	}
	virtual int64 GetPTS() const override
	{
		return PTS;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual const FVideoDecoderAllocFrameBufferResult* GetAllocatedBuffer() const
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return &Buffer;
	}

	virtual int32 GetCropLeft() const override
	{
		return 0;
	}
	virtual int32 GetCropRight() const override
	{
		return 0;
	}
	virtual int32 GetCropTop() const override
	{
		return 0;
	}
	virtual int32 GetCropBottom() const override
	{
		return 0;
	}
	virtual int32 GetAspectX() const override
	{
		return 1;
	}
	virtual int32 GetAspectY() const override
	{
		return 1;
	}
	virtual int32 GetPitchX() const override
	{
		return Width;
	}
	virtual int32 GetPitchY() const override
	{
		return Height;
	}
	virtual uint32 GetColorFormat() const override
	{
		return 0;
	}

	// Internal for allocation.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderAllocFrameBufferResult* GetBuffer()
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return &Buffer;
	}

private:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderAllocFrameBufferResult	Buffer = {};
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	int32	RefCount = 1;
	int32	Width = 0;
	int32	Height = 0;
	int64	PTS = 0;
};

	// query whether or not encoder is supported and available
//	static bool GetIsAvailable(FVideoEncoderInputImpl& InInput, FVideoEncoderInfo& OutEncoderInfo);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FVideoDecoderH264_Dummy::Register(FVideoDecoderFactory& InFactory)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderInfo	DecoderInfo;
	DecoderInfo.CodecType = ECodecType::H264;
	DecoderInfo.MaxWidth = 1920;
	DecoderInfo.MaxHeight = 1088;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InFactory.Register(DecoderInfo, []() {
		return new FVideoDecoderH264_Dummy();
	});
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVideoDecoderH264_Dummy::FVideoDecoderH264_Dummy()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVideoDecoderH264_Dummy::~FVideoDecoderH264_Dummy()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FVideoDecoderH264_Dummy::Setup(const FVideoDecoder::FInit& InInit)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CreateDecoderAllocationInterfaceFN = InInit.CreateDecoderAllocationInterface;
	ReleaseDecoderAllocationInterfaceFN = InInit.ReleaseDecoderAllocationInterface;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return true;
}

void FVideoDecoderH264_Dummy::Shutdown()
{
	if (bIsInitialized)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ReleaseDecoderAllocationInterface();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bIsInitialized = false;
	}
	delete this;
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVideoDecoder::EDecodeResult FVideoDecoderH264_Dummy::Decode(const FVideoDecoderInput* InInput)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	// Lazily initialize the decoder.
	if (!bIsInitialized)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bIsInitialized = CreateDecoderAllocationInterface();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	

	//
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (OnDecodedFrame)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		if (bIsInitialized)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			int32_t	Width = InInput->GetWidth();
			int32_t Height = InInput->GetHeight();
			
			TUniquePtr<FVideoDecoderOutputDummy> pNew(new FVideoDecoderOutputDummy(Width, Height, InInput->GetPTS()));


			// Get memory from the application
			FVideoDecoderAllocFrameBufferParams ap {};
			EFrameBufferAllocReturn ar;
			ap.FrameBufferType = EFrameBufferType::CODEC_RawBuffer;
			ap.AllocSize = Width * Height * 3 / 2;
			ap.AllocAlignment = 16;
			ap.AllocFlags = 0;
			ap.Width = Width;
			ap.Height = Height;
			ap.BytesPerPixel = 1;
			ar = AllocateOutputFrameBuffer(pNew->GetBuffer(), &ap);
			if (ar == EFrameBufferAllocReturn::CODEC_Success)
			{
				// Got an output buffer.
				check(pNew->GetBuffer()->AllocatedBuffer);
			}
			else if (ar == EFrameBufferAllocReturn::CODEC_TryAgainLater)
			{
				check(!"TODO");
				return FVideoDecoder::EDecodeResult::Failure;
//				return FVideoDecoder::EDecodeResult::TryAgainLater;
			}
			else
			{
				// Error!
				return FVideoDecoder::EDecodeResult::Failure;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			// Buffer needs to contain an NV12 texture.
			int32 ySize = Width * Height;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			uint8* pY = (uint8*)pNew->GetBuffer()->AllocatedBuffer;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			uint8* pUV = pY + ySize;
			FMemory::Memset(pUV, 128, ySize / 2);
			int C = ++YOffset;
			for(int32 y=0,yMax=Height; y<yMax; ++y)
			{
				FMemory::Memset(pY, (uint8)C++, Width);
				pY += Width;
			}
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OnDecodedFrame(pNew.Release());
			return FVideoDecoder::EDecodeResult::Success;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		else
		{
// TODO: could not decode, but may need to still notify the caller with an empty frame??
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return FVideoDecoder::EDecodeResult::Failure;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
	else
	{
		// With no registered callback that's interested in the result we can presume we would have been successful.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return FVideoDecoder::EDecodeResult::Success;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}


} // namespace AVEncoder


#endif // AVENCODER_VIDEO_DECODER_AVAILABLE_H264_DUMMY
