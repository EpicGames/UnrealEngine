// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define AVENCODER_VIDEO_DECODER_AVAILABLE_H264_DUMMY		// enable dummy h264 decoder

#ifdef AVENCODER_VIDEO_DECODER_AVAILABLE_H264_DUMMY

#include "VideoDecoderFactory.h"


namespace AVEncoder
{

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FVideoDecoderH264_Dummy : public FVideoDecoder
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	// register decoder with video decoder factory
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static void Register(FVideoDecoderFactory& InFactory);
	virtual bool Setup(const FInit& InInit) override;
	virtual EDecodeResult Decode(const FVideoDecoderInput* InInput) override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void Shutdown() override;



	// query whether or not encoder is supported and available
//	static bool GetIsAvailable(FVideoEncoderInputImpl& InInput, FVideoEncoderInfo& OutEncoderInfo);


private:
	FVideoDecoderH264_Dummy();
	virtual ~FVideoDecoderH264_Dummy();
	uint8_t YOffset = 0;
	bool bIsInitialized = false;
};

} // namespace AVEncoder

#endif // AVENCODER_VIDEO_DECODER_AVAILABLE_H264_DUMMY
