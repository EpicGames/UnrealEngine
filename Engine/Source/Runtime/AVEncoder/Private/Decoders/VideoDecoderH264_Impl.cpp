// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoderH264_Impl.h"

#ifndef CODEC_HAVE_NATIVE_H264_DECODER
#define CODEC_HAVE_NATIVE_H264_DECODER 0
#endif

#if PLATFORM_WINDOWS
#undef CODEC_HAVE_NATIVE_H264_DECODER
#define CODEC_HAVE_NATIVE_H264_DECODER 1
#include "Windows/VideoDecoderH264_Windows.h"
#endif


namespace AVEncoder
{



#if CODEC_HAVE_NATIVE_H264_DECODER

#if PLATFORM_WINDOWS
PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FVideoDecoderH264_Impl::Register(FVideoDecoderFactory& InFactory)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	FVideoDecoderH264_Windows::Register(InFactory);
}
#endif

#else

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FVideoDecoderH264_Impl::Register(FVideoDecoderFactory& InFactory)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
}

#endif

} // namespace AVEncoder
