// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoderMpeg4.h"

#include <CoreMinimal.h>

#include "Containers/Queue.h"

#include "VideoDecoderCommon.h"
#include "VideoDecoderAllocationTypes.h"
#include "VideoDecoderUtilities.h"

#include "vdecmpeg4/vdecmpeg4.h"
#include "vdecmpeg4/vdecmpeg4_Stream.h"


namespace AVEncoder
{

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FVideoDecoderOutputMPEG4 : public FVideoDecoderOutput
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderOutputMPEG4(int32 w, int32 h, int64 pts)
	: Width(w), Height(h), PTS(pts)
	{}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FVideoDecoderOutputMPEG4()
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual int32 AddRef() override
	{
		return FPlatformAtomics::InterlockedIncrement(&RefCount);
	}

	virtual int32 Release() override
	{
		int32 c = FPlatformAtomics::InterlockedDecrement(&RefCount);
		// We do not release the allocated buffer from the application here.
		// This is meant to release only what the decoder uses internally, but not the
		// external buffers the application is working with!
		if (c == 0)
		{
			delete this;
		}
		return c;
	}

	virtual int32 GetWidth() const override
	{
		return Width;
	}
	virtual int32 GetHeight() const override
	{
		return Height;
	}
	virtual int64 GetPTS() const override
	{
		return PTS;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual const FVideoDecoderAllocFrameBufferResult* GetAllocatedBuffer() const
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return &Buffer;
	}

	virtual int32 GetCropLeft() const override
	{
		return 0;
	}
	virtual int32 GetCropRight() const override
	{
		return 0;
	}
	virtual int32 GetCropTop() const override
	{
		return 0;
	}
	virtual int32 GetCropBottom() const override
	{
		return 0;
	}
	virtual int32 GetAspectX() const override
	{
		return 1;
	}
	virtual int32 GetAspectY() const override
	{
		return 1;
	}
	void SetPitchX(int32 InPitchX)
	{
		Pitch = InPitchX;
	}
	virtual int32 GetPitchX() const override
	{
		return Pitch;
	}
	virtual int32 GetPitchY() const override
	{
		return Height;
	}
	virtual uint32 GetColorFormat() const override
	{
		return 0;
	}

	// Internal for allocation.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderAllocFrameBufferResult* GetBuffer()
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return &Buffer;
	}

private:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderAllocFrameBufferResult	Buffer = {};
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	int32	RefCount = 1;
	int32	Width = 0;
	int32	Pitch = 0;
	int32	Height = 0;
	int64	PTS = 0;
};







static uint32 sAllocSize = 0;
static TMap<void*, uint32>& Actives()
{
	static TMap<void*, uint32> a;
	return a;
}
class FVideoDecoderMPEG4_Impl : public FVideoDecoderMPEG4, public vdecmpeg4::VIDStreamIO, public vdecmpeg4::VIDStreamEvents
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual bool Setup(const FInit& InInit) override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual void Shutdown() override;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual EDecodeResult Decode(const FVideoDecoderInput* InInput) override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FVideoDecoderMPEG4_Impl();
	virtual ~FVideoDecoderMPEG4_Impl();

protected:
	static void* vidMalloc(uint32_t size, uint32_t alignment)
	{ 
		void* Addr = FMemory::MallocZeroed(size, alignment); 
		Actives().Emplace(Addr, size);
		sAllocSize += size;
		return Addr;
	}
	static void vidFree(void* block)
	{ 
		if (block)
		{
			uint32 s = Actives().FindAndRemoveChecked(block);
			sAllocSize -= s;
			FMemory::Free(block); 
		}
	}
	static void vidReport(const char* pMessage)
	{
		FString m(ANSI_TO_TCHAR(pMessage));
		UE_LOGF(LogVideoDecoder, Log, "%ls", *m);
	}

	virtual void FoundVideoObjectLayer(const VOLInfo& volInfo) override;
	virtual vdecmpeg4::VIDStreamResult Read(uint8_t* pRequestedDataBuffer, uint32_t requestedDataBytes, uint32_t& actualDataBytes) override;
	virtual bool IsEof() override;

	bool FirstUseInit();


	struct FInDecoderData
	{
		TArray<uint8>	Data;
		int64			PTS = 0;
		int32			Width = 0;
		int32			Height = 0;
		int32			DataOffset = 0;
		bool			bIsKeyframe = false;
		bool			bIsComplete = false;
	};

	TQueue<TUniquePtr<FInDecoderData>, EQueueMode::Spsc>	PendingDecodeData;
	TUniquePtr<FInDecoderData>	CurrentAU;

	vdecmpeg4::VIDDecoderSetup	DecoderSetup;
	vdecmpeg4::VIDDecoder		DecoderHandle;
	vdecmpeg4::VIDError			LastDecoderError;
	bool						bIsInitialized;
	bool						bDataReaderAttached;
};



PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FVideoDecoderMPEG4::Register(FVideoDecoderFactory& InFactory)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderInfo	DecoderInfo;
	DecoderInfo.CodecType = ECodecType::MPEG4;
	DecoderInfo.MaxWidth = 1920;
	DecoderInfo.MaxHeight = 1088;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	InFactory.Register(DecoderInfo, []() {
		return new FVideoDecoderMPEG4_Impl();
	});
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

int32 gTestStreamIndex4 = 0;

FVideoDecoderMPEG4_Impl::FVideoDecoderMPEG4_Impl()
{
	DecoderSetup = {};
	DecoderHandle = nullptr;
	LastDecoderError = vdecmpeg4::VID_OK;
	bIsInitialized = false;
	bDataReaderAttached = false;
}

FVideoDecoderMPEG4_Impl::~FVideoDecoderMPEG4_Impl()
{
	check(!DecoderHandle);
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FVideoDecoderMPEG4_Impl::Setup(const FVideoDecoder::FInit& InInit)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CreateDecoderAllocationInterfaceFN = InInit.CreateDecoderAllocationInterface;
	ReleaseDecoderAllocationInterfaceFN = InInit.ReleaseDecoderAllocationInterface;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return true;
}

void FVideoDecoderMPEG4_Impl::Shutdown()
{
	if (bIsInitialized)
	{
		if (DecoderHandle)
		{
			vdecmpeg4::VIDDestroyDecoder(DecoderHandle);
			DecoderHandle = nullptr;
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ReleaseDecoderAllocationInterface();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		bIsInitialized = false;
	}
	delete this;
}

bool FVideoDecoderMPEG4_Impl::FirstUseInit()
{
	if (!bIsInitialized)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		bIsInitialized = CreateDecoderAllocationInterface();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (bIsInitialized)
		{
			FMemory::Memzero(DecoderSetup);
			DecoderSetup.size = sizeof(DecoderSetup);
			DecoderSetup.width = 0;
			DecoderSetup.height = 0;
			DecoderSetup.flags = vdecmpeg4::VID_DECODER_VID_BUFFERS;
			DecoderSetup.numOfVidBuffers = 5;
			DecoderSetup.cbMemAlloc = vidMalloc;
			DecoderSetup.cbMemFree = vidFree;
			DecoderSetup.cbReport = vidReport;
			if ((LastDecoderError = vdecmpeg4::VIDCreateDecoder(&DecoderSetup, &DecoderHandle)) == vdecmpeg4::VID_OK)
			{
				bIsInitialized = true;
			}
			else
			{
				UE_LOGF(LogVideoDecoder, Error, "VIDCreateDecoder() failed with %d", (int)LastDecoderError);
			}
		}
	}
	return bIsInitialized;
}


void FVideoDecoderMPEG4_Impl::FoundVideoObjectLayer(const VOLInfo& volInfo)
{
	// No-op.
}

vdecmpeg4::VIDStreamResult FVideoDecoderMPEG4_Impl::Read(uint8_t* pRequestedDataBuffer, uint32_t requestedDataBytes, uint32_t& actualDataBytes)
{
	actualDataBytes = 0;
	if (CurrentAU.IsValid())
	{
		const uint8* Base = CurrentAU->Data.GetData();
		int32 Size = CurrentAU->Data.Num();
		int32& Offset = CurrentAU->DataOffset;
		// Trying to read past the size of this access unit means it is done.
		if (Offset >= Size)
		{
			return vdecmpeg4::VID_STREAM_EOF;
		}
		else if ((int32)requestedDataBytes <= Size - Offset)
		{
			FMemory::Memcpy(pRequestedDataBuffer, Base + Offset, requestedDataBytes);
			actualDataBytes = requestedDataBytes;
			Offset += requestedDataBytes;
			return vdecmpeg4::VID_STREAM_OK;
		}
		else
		{
			// Reading the last bytes requires padding to 32 bit with 0 bytes.
			uint32 remain = Size - Offset;
			uint32 numPadding = (4 - (remain & 3)) & 3;
			FMemory::Memcpy(pRequestedDataBuffer, Base + Offset, remain);
			if (numPadding)
			{
				FMemory::Memzero(pRequestedDataBuffer + remain, numPadding);
			}
			actualDataBytes = remain + numPadding;
			Offset = Size;
			return vdecmpeg4::VID_STREAM_OK;
		}
	}
	return vdecmpeg4::VID_STREAM_ERROR;
}

bool FVideoDecoderMPEG4_Impl::IsEof()
{
	return CurrentAU.IsValid() ? CurrentAU->DataOffset >= CurrentAU->Data.Num() : false;
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
static void CopyI420ToNV12(const FVideoDecoderAllocFrameBufferResult* OutBuf, const vdecmpeg4::VIDImage* vid)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	int32 Width = vid->width;
	int32 Height = vid->height;

	const uint8_t* srcY = vid->y;
	const uint8_t* srcU = vid->u;
	const uint8_t* srcV = vid->v;

	// Allocated buffer needs to have 3 planes.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(OutBuf->AllocatedPlanesNum == 3);

	uint8* OutBufferBase = (uint8*)OutBuf->AllocatedBuffer;

	check(OutBuf->AllocatedPlaneDesc[0].BytesPerPixel == 1);
	check(OutBuf->AllocatedPlaneDesc[0].ByteOffsetBetweenPixels == 1);
	uint8* dstY = OutBufferBase + OutBuf->AllocatedPlaneDesc[0].ByteOffsetToFirstPixel;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// Need to copy the Y plane row by row.
	for(int32 y=0; y<Height; ++y)
	{
		FMemory::Memcpy(dstY, srcY, Width);
		srcY += vid->texWidth;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		dstY += OutBuf->AllocatedPlaneDesc[0].ByteOffsetBetweenRows;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	

	// The U and V plane must be interleaved for NV12. We don't specifically do interleaving here but
	// instead rely on the output plane description to be set up accordingly.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	check(OutBuf->AllocatedPlaneDesc[1].BytesPerPixel == 1);
	check(OutBuf->AllocatedPlaneDesc[2].BytesPerPixel == 1);
	uint8* dstU = OutBufferBase + OutBuf->AllocatedPlaneDesc[1].ByteOffsetToFirstPixel;
	uint8* dstV = OutBufferBase + OutBuf->AllocatedPlaneDesc[2].ByteOffsetToFirstPixel;
	const int32 uOffCol = OutBuf->AllocatedPlaneDesc[1].ByteOffsetBetweenPixels;
	const int32 vOffCol = OutBuf->AllocatedPlaneDesc[2].ByteOffsetBetweenPixels;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	for(int32 v=0, vMax=Height/2; v<vMax; ++v)
	{
		uint8* U = dstU;
		uint8* V = dstV;
		for(int32 u=0, uMax=Width/2; u<uMax; ++u)
		{
			*U = *srcU++;
			*V = *srcV++;
			U += uOffCol;
			V += vOffCol;
		}
		srcU += (vid->texWidth - vid->width) / 2;
		srcV += (vid->texWidth - vid->width) / 2;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		dstU += OutBuf->AllocatedPlaneDesc[1].ByteOffsetBetweenRows;
		dstV += OutBuf->AllocatedPlaneDesc[2].ByteOffsetBetweenRows;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVideoDecoder::EDecodeResult FVideoDecoderMPEG4_Impl::Decode(const FVideoDecoderInput* InInput)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	// With no registered callback that's interested in the result we can presume we would have been successful.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (!OnDecodedFrame)
	{
		return FVideoDecoder::EDecodeResult::Success;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// Initialize decoder on first decode call.
	if (!FirstUseInit())
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return FVideoDecoder::EDecodeResult::Failure;
	}PRAGMA_ENABLE_DEPRECATION_WARNINGS
	

	// Setup an access unit to run through the decoder.
	TUniquePtr<FInDecoderData> AU = MakeUnique<FInDecoderData>();
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (InInput->GetDataSize() <= 0)
	{
		return FVideoDecoder::EDecodeResult::Failure;
	}
	AU->Data.AddUninitialized(InInput->GetDataSize());
	FMemory::Memcpy(AU->Data.GetData(), InInput->GetData(), InInput->GetDataSize());
	AU->DataOffset = 0;
	AU->PTS = InInput->GetPTS();
	AU->Width = InInput->GetWidth();
	AU->Height = InInput->GetHeight();
	AU->bIsKeyframe = InInput->IsKeyframe();
	AU->bIsComplete = InInput->IsCompleteFrame();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	PendingDecodeData.Enqueue(MoveTemp(AU));

	// Decode all pending input.
	while(1)
	{
		// Need a new access unit?
		if (!CurrentAU.IsValid() && !PendingDecodeData.Dequeue(CurrentAU))
		{
			break;
		}

		// Invoke decoder.
		const vdecmpeg4::VIDImage* frame = nullptr;
		if (!bDataReaderAttached)
		{
			if ((LastDecoderError = vdecmpeg4::VIDStreamSet(DecoderHandle, this, this)) == vdecmpeg4::VID_OK)
			{
				bDataReaderAttached = true;
			}
			else
			{
				UE_LOGF(LogVideoDecoder, Error, "VIDCreateDecoder() failed with %d", (int)LastDecoderError);
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				return FVideoDecoder::EDecodeResult::Failure;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				//vdecmpeg4::VIDDestroyDecoder(DecoderHandle);
				//DecoderHandle = nullptr;
			}
		}

		LastDecoderError = vdecmpeg4::VIDStreamDecode(DecoderHandle, 0.0f, &frame);
		if (LastDecoderError == vdecmpeg4::VID_OK)
		{
			if (frame)
			{
				int32_t	Width = frame->width;
				int32_t Height = frame->height;

				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				TUniquePtr<FVideoDecoderOutputMPEG4> pNew(new FVideoDecoderOutputMPEG4(Width, Height, InInput->GetPTS()));

				// Get memory from the application
				FVideoDecoderAllocFrameBufferParams ap {};
				EFrameBufferAllocReturn ar;
				ap.FrameBufferType = EFrameBufferType::CODEC_RawBuffer;
				ap.AllocSize = Width * Height * 3 / 2;
				ap.AllocAlignment = 16;
				ap.AllocFlags = 0;
				ap.Width = Width;
				ap.Height = Height;
				ap.BytesPerPixel = 1;
				// Check for valid values.
				if (Width <= 0 || Height <= 0 || ap.AllocSize <= 0)
				{
					return FVideoDecoder::EDecodeResult::Failure;
				}
				ar = AllocateOutputFrameBuffer(pNew->GetBuffer(), &ap);
				if (ar == EFrameBufferAllocReturn::CODEC_Success)
				{
					// Got an output buffer.
					check(pNew->GetBuffer()->AllocatedBuffer);
				}
				else if (ar == EFrameBufferAllocReturn::CODEC_TryAgainLater)
				{
					// Try again later is not supported. We are realtime here and there's no "later"
					return FVideoDecoder::EDecodeResult::Failure;
				}
				else
				{
					// Error!
					return FVideoDecoder::EDecodeResult::Failure;
				}

				if (pNew->GetBuffer()->AllocatedPlanesNum >= 1)
				{
					pNew->SetPitchX(pNew->GetBuffer()->AllocatedPlaneDesc[0].Width);
				}
				else
				{
					pNew->SetPitchX(ap.Width);
				}
				PRAGMA_ENABLE_DEPRECATION_WARNINGS

				// Copy the image across, converting it into NV12 format.
				CopyI420ToNV12(pNew->GetBuffer(), frame);
				frame->Release();
				// Deliver
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				OnDecodedFrame(pNew.Release());
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}	
		}
		else if (LastDecoderError == vdecmpeg4::VID_ERROR_STREAM_UNDERFLOW)
		{
			// Not enough data. Keep going.
		}
		else if (LastDecoderError == vdecmpeg4::VID_ERROR_STREAM_EOF)
		{
			// Access unit was fully consumed.
		}
		else
		{
			// Error!
			UE_LOGF(LogVideoDecoder, Error, "VIDStreamDecode() failed with %d", (int)LastDecoderError);
// TODO: destroy decoder and wait for next keyframe?
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return FVideoDecoder::EDecodeResult::Failure;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}

		// Are we done with the current access unit?
		if (CurrentAU->DataOffset >= CurrentAU->Data.Num())
		{
			CurrentAU.Reset();
		}
	}
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FVideoDecoder::EDecodeResult::Success;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


} // namespace AVEncoder

