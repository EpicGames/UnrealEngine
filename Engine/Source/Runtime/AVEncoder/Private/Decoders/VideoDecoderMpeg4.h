// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VideoDecoderFactory.h"


namespace AVEncoder
{

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FVideoDecoderMPEG4 : public FVideoDecoder
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	// register decoder with video decoder factory
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static void Register(FVideoDecoderFactory& InFactory);
	virtual bool Setup(const FInit& InInit) = 0;
	virtual EDecodeResult Decode(const FVideoDecoderInput* InInput) = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void Shutdown() = 0;


protected:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FVideoDecoderMPEG4() = default;
	FVideoDecoderMPEG4() = default;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
};

} // namespace AVEncoder
