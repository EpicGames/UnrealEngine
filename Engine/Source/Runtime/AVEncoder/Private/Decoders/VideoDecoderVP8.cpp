// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoderVP8.h"


// Should be defined in AVEncoder_{Platform}.Build.cs
// If not there is no native support and we have to provide a default implementation
// that registers nothing.
#ifndef CODEC_HAVE_NATIVE_VP8_DECODER
#define CODEC_HAVE_NATIVE_VP8_DECODER 0
#endif


#if CODEC_HAVE_NATIVE_VP8_DECODER == 0

namespace AVEncoder
{

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FVideoDecoderVP8::Register(FVideoDecoderFactory& InFactory)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
}

} // namespace AVEncoder

#endif
