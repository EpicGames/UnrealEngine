// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoderH264_Windows.h"

#include <CoreMinimal.h>
#include <HAL/Thread.h>

#include "MicrosoftCommon.h"

#include "VideoDecoderCommon.h"
#include "VideoDecoderAllocationTypes.h"
#include "VideoDecoderUtilities.h"

#include "Containers/Queue.h"
#include "Misc/ScopeLock.h"
#include "Templates/RefCounting.h"


#define VERIFY_HR(FNcall,...)						\
Result = FNcall;									\
if (FAILED(Result))									\
{													\
	UE_LOG(LogVideoDecoder, Error, __VA_ARGS__);	\
	return false;									\
}


namespace AVEncoder
{

namespace
{

// Define necessary GUIDs ourselves to avoid pulling in a lib that just has these and nothing else we need.
static const GUID MFTmsH264Decoder 		 = { 0x62CE7E72, 0x4C71, 0x4D20, { 0xB1, 0x5D, 0x45, 0x28, 0x31, 0xA8, 0x7D, 0x9D } };
#if (WINVER < _WIN32_WINNT_WIN8)
static const GUID MF_SA_D3D11_AWARE = { 0x206b4fc8, 0xfcf9, 0x4c51, { 0xaf, 0xe3, 0x97, 0x64, 0x36, 0x9e, 0x33, 0xa0 } };
#endif

}

/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/

namespace Electra
{

bool IsWindows8Plus()
{
	return FPlatformMisc::VerifyWindowsVersion(6, 2);
}

bool IsWindows7Plus()
{
	return FPlatformMisc::VerifyWindowsVersion(6, 1);
}

struct FParamDict
{
};

}

class IDecoderOutput;

class IDecoderOutputOwner
{
public:
	virtual ~IDecoderOutputOwner() = default;
	virtual void SampleReleasedToPool(IDecoderOutput* InDecoderOutput) = 0;
};

class FNativeVideoDecoderOutput
{
public:
	virtual ~FNativeVideoDecoderOutput() = default;

	virtual void SetOwner(const TSharedPtr<IDecoderOutputOwner, ESPMode::ThreadSafe>& Renderer) = 0;
	virtual void InitializePoolable() { }
	virtual void ShutdownPoolable() { }

	virtual FIntPoint GetDim() const = 0;
	virtual FTimespan GetDuration() const
	{
		return 0;
	}
};

class FVideoDecoderOutputDX : public FNativeVideoDecoderOutput
{
public:
	virtual ~FVideoDecoderOutputDX() = default;

	enum class EOutputType
	{
		Unknown = 0,
		SoftwareWin7,			// SW decode into buffer
		SoftwareWin8Plus,		// SW decode into DX11 texture
		HardwareWin8Plus,		// HW decode into shared DX11 texture
		HardwareDX9_DX12,		// HW decode into buffer
	};

	virtual EOutputType GetOutputType() const = 0;

	virtual TRefCountPtr<IMFSample> GetMFSample() const = 0;

	virtual const TArray<uint8> & GetBuffer() const = 0;
	virtual uint32 GetStride() const = 0;

	virtual TRefCountPtr<ID3D11Texture2D> GetTexture() const = 0;
	virtual TRefCountPtr<ID3D11Device> GetDevice() const = 0;
};

class FF5PlayerVideoDecoderOutputDX : public FVideoDecoderOutputDX
{
public:
	FF5PlayerVideoDecoderOutputDX()
		: OutputType(EOutputType::Unknown)
		, SampleDim(0,0)
		, Stride(0)
	{
	}

	~FF5PlayerVideoDecoderOutputDX()
	{
		// We use this without a pool, so we need to shutdown it now.
		ShutdownPoolable();
	}

	// Hardware decode to buffer (DX9 and DX12)
	void InitializeWithBuffer(uint32 InStride, FIntPoint Dim);

	// Software decode to buffer (Win8+ DX11)
	void InitializeWithTextureBuffer(uint32 InStride, FIntPoint Dim);

	// Hardware decode to shared DX11 texture (Win8+ DX11)
	bool InitializeWithSharedTexture(const TRefCountPtr<ID3D11Device>& InD3D11Device, const TRefCountPtr<ID3D11DeviceContext> InDeviceContext, const TRefCountPtr<IMFSample>& MFSample, const FIntPoint& OutputDim);

	// Software decode (into a buffer via a temporary MFCreateMemoryBuffer and MFSample _or_ application created texture and MFSample)
	bool PreInitForSoftwareDecode(FIntPoint OutputDim, ID3D11Texture2D* TextureBuffer);

	void SetOwner(const TSharedPtr<IDecoderOutputOwner, ESPMode::ThreadSafe>& InOwningRenderer) override
	{
	}

	void ShutdownPoolable() override final;

	virtual EOutputType GetOutputType() const override
	{
		return OutputType;
	}

	virtual TRefCountPtr<IMFSample> GetMFSample() const override
	{
		check(OutputType == EOutputType::SoftwareWin8Plus || OutputType == EOutputType::SoftwareWin7);
		return MFSample;
	}

	virtual const TArray<uint8>& GetBuffer() const override
	{
		check(!"Should not be called!");
		static TArray<uint8> Buffer;
		return Buffer;
	}

	virtual uint32 GetStride() const override
	{
		return Stride;
	}

	virtual TRefCountPtr<ID3D11Texture2D> GetTexture() const override
	{
		check(OutputType == EOutputType::SoftwareWin8Plus || OutputType == EOutputType::HardwareWin8Plus);
		return SharedTexture;
	}

	virtual TRefCountPtr<ID3D11Device> GetDevice() const override
	{
		check(OutputType == EOutputType::SoftwareWin8Plus || OutputType == EOutputType::HardwareWin8Plus);
		return D3D11Device;
	}

	virtual FIntPoint GetDim() const override
	{
		return SampleDim;
	}

private:
	// Decoder output type
	EOutputType OutputType;

	// The texture shared between our decoding device here and the application render device.
	TRefCountPtr<ID3D11Texture2D> SharedTexture;
	TRefCountPtr<ID3D11Device> D3D11Device;

	// An output media sample we use for SW decoding that either wraps a CPU buffer or a texture buffer
	// from the texture allocated by the application.
	TRefCountPtr<IMFSample> MFSample;

	// Dimension of any internally allocated buffer - stored explicitly to cover various special cases for DX
	FIntPoint SampleDim;
	uint32 Stride;
};


bool FF5PlayerVideoDecoderOutputDX::PreInitForSoftwareDecode(FIntPoint OutputDim, ID3D11Texture2D* TextureBuffer)
{
	FIntPoint Dim;
	Dim.X = OutputDim.X;
	Dim.Y = OutputDim.Y * 3 / 2;	// adjust height to encompass Y and UV planes

	EOutputType NewOutputType = TextureBuffer ? EOutputType::SoftwareWin8Plus : EOutputType::SoftwareWin7;

	bool bNeedNew = !MFSample.IsValid() || SampleDim != Dim || NewOutputType != OutputType;
	OutputType = NewOutputType;

	SharedTexture = nullptr;
	if (bNeedNew)
	{
		TRefCountPtr<IMFMediaBuffer> MediaBuffer;

		if (TextureBuffer)
		{
			TRefCountPtr<ID3D11Texture2D> ApplicationTexture(TextureBuffer, true);

			if (FAILED(MFCreateDXGISurfaceBuffer(__uuidof(ID3D11Texture2D), ApplicationTexture, 0, false, MediaBuffer.GetInitReference())))
			{
				UE_LOGF(LogVideoDecoder, Error, "Failed to create decoder surface buffer from texture");
				return false;
			}
		}
		else
		{
			SampleDim = Dim;
			MFSample = nullptr;
			// SW decode results are just delivered in a simple CPU-side buffer. Create the decoder side version of this...
			if (MFCreateMemoryBuffer(Dim.X * Dim.Y * sizeof(uint8), MediaBuffer.GetInitReference()) != S_OK)
			{
				UE_LOGF(LogVideoDecoder, Error, "Failed to create a decoder memory buffer");
				return false;
			}
		}
	
		if (MFCreateSample(MFSample.GetInitReference()) == S_OK)
		{
			if (FAILED(MFSample->AddBuffer(MediaBuffer)))
			{
				UE_LOGF(LogVideoDecoder, Error, "Failed to add buffer to decoder output media sample");
				return false;
			}
		}
		else
		{
			UE_LOGF(LogVideoDecoder, Error, "Failed to create a decoder media sample");
			return false;
		}
	}
	return true;
}

void FF5PlayerVideoDecoderOutputDX::InitializeWithBuffer(uint32 InStride, FIntPoint Dim)
{
	OutputType = EOutputType::HardwareDX9_DX12;
	SampleDim = Dim;
	Stride = InStride;
}

void FF5PlayerVideoDecoderOutputDX::InitializeWithTextureBuffer(uint32 InStride, FIntPoint Dim)
{
	OutputType = EOutputType::SoftwareWin8Plus;
	SampleDim = Dim;
	Stride = InStride;
}

bool FF5PlayerVideoDecoderOutputDX::InitializeWithSharedTexture(const TRefCountPtr<ID3D11Device>& InD3D11Device, const TRefCountPtr<ID3D11DeviceContext> InDeviceContext, const TRefCountPtr<IMFSample>& InMFSample, const FIntPoint& OutputDim)
{
	HRESULT Result = S_OK;
	OutputType = EOutputType::HardwareWin8Plus;

	bool bNeedsNew = !SharedTexture.IsValid() || (SampleDim.X != OutputDim.X || SampleDim.Y != OutputDim.Y);

	if (bNeedsNew)
	{
		SampleDim = OutputDim;

		// Make a texture we pass on as output
		D3D11_TEXTURE2D_DESC TextureDesc;
		TextureDesc.Width = SampleDim.X;
		TextureDesc.Height = SampleDim.Y;
		TextureDesc.MipLevels = 1;
		TextureDesc.ArraySize = 1;
		TextureDesc.Format = DXGI_FORMAT_NV12;
		TextureDesc.SampleDesc.Count = 1;
		TextureDesc.SampleDesc.Quality = 0;
		TextureDesc.Usage = D3D11_USAGE_DEFAULT;
		TextureDesc.BindFlags = 0;
		TextureDesc.CPUAccessFlags = 0;
		TextureDesc.MiscFlags = D3D11_RESOURCE_MISC_SHARED_KEYEDMUTEX;
		if (FAILED(Result = InD3D11Device->CreateTexture2D(&TextureDesc, nullptr, SharedTexture.GetInitReference())))
		{
			UE_LOGF(LogVideoDecoder, Error, "ID3D11Device::CreateTexture2D() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}
		D3D11Device = InD3D11Device;
	}

	// If we got a texture, copy the data from the decoder into it...
	if (SharedTexture)
	{
		// Get output texture from decoder...
		TRefCountPtr<IMFMediaBuffer> MediaBuffer;
		if (FAILED(Result = InMFSample->GetBufferByIndex(0, MediaBuffer.GetInitReference())))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFSample::GetBufferByIndex() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}
		TRefCountPtr<IMFDXGIBuffer> DXGIBuffer;
		if (FAILED(Result = MediaBuffer->QueryInterface(__uuidof(IMFDXGIBuffer), (void**)DXGIBuffer.GetInitReference())))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFMediaBuffer::QueryInterface(IMFDXGIBuffer) failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}
		TRefCountPtr<ID3D11Texture2D> DecoderTexture;
		if (FAILED(Result = DXGIBuffer->GetResource(IID_PPV_ARGS(DecoderTexture.GetInitReference()))))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFDXGIBuffer::GetResource() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}
		uint32 ViewIndex = 0;
		if (FAILED(Result = DXGIBuffer->GetSubresourceIndex(&ViewIndex)))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFDXGIBuffer::GetSubresourceIndex() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}

		// Source data may be larger than desired output, but crop area will be aligned such that we can always copy from 0,0
		D3D11_BOX SrcBox;
		SrcBox.left = 0;
		SrcBox.top = 0;
		SrcBox.front = 0;
		SrcBox.right = OutputDim.X;
		SrcBox.bottom = OutputDim.Y;
		SrcBox.back = 1;

		TRefCountPtr<IDXGIKeyedMutex> KeyedMutex;
		Result = SharedTexture->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)&KeyedMutex);
		if (KeyedMutex)
		{
			// No wait on acquire since sample is new and key is 0.
			if (SUCCEEDED(Result = KeyedMutex->AcquireSync(0, 0)))
			{
				// Copy texture using the decoder DX11 device... (and apply any cropping - see above note)
				InDeviceContext->CopySubresourceRegion(SharedTexture, 0, 0, 0, 0, DecoderTexture, ViewIndex, &SrcBox);
				// Mark texture as updated with key of 1
				// Sample will be read in Convert() method of texture sample
				KeyedMutex->ReleaseSync(1);
			}
		}

		// Make sure texture is updated before giving access to the sample in the rendering thread.
		InDeviceContext->Flush();
	}
	return true;
}

void FF5PlayerVideoDecoderOutputDX::ShutdownPoolable()
{
	if (OutputType == EOutputType::HardwareWin8Plus)
	{
		// Correctly release the keyed mutex when the sample is returned to the pool
		TRefCountPtr<IDXGIResource> OtherResource(nullptr);
		if (SharedTexture)
		{
			SharedTexture->QueryInterface(__uuidof(IDXGIResource), (void**)&OtherResource);
		}

		if (OtherResource)
		{
			HANDLE SharedHandle = nullptr;
			if (SUCCEEDED(OtherResource->GetSharedHandle(&SharedHandle)))
			{
				TRefCountPtr<ID3D11Resource> SharedResource;
				D3D11Device->OpenSharedResource(SharedHandle, __uuidof(ID3D11Texture2D), (void**)&SharedResource);
				if (SharedResource)
				{
					TRefCountPtr<IDXGIKeyedMutex> KeyedMutex;
					OtherResource->QueryInterface(_uuidof(IDXGIKeyedMutex), (void**)&KeyedMutex);

					if (KeyedMutex)
					{
						// Reset keyed mutex
						if (SUCCEEDED(KeyedMutex->AcquireSync(1, 0)))
						{
							// Texture was never read
							KeyedMutex->ReleaseSync(0);
						}
						else if (SUCCEEDED(KeyedMutex->AcquireSync(2, 0)))
						{
							// Texture was read at least once
							KeyedMutex->ReleaseSync(0);
						}
					}
				}
			}
		}
	}
}
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/
/***************************************************************************************************************************************************/



/**
 * Decoded media sample
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FVideoDecoderOutputH264_Windows : public FVideoDecoderOutput
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderOutputH264_Windows()
	{}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FVideoDecoderOutputH264_Windows()
	{}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual int32 AddRef() override
	{
		return FPlatformAtomics::InterlockedIncrement(&RefCount);
	}

	virtual int32 Release() override
	{
		int32 c = FPlatformAtomics::InterlockedDecrement(&RefCount);
		// We do not release the allocated buffer from the application here.
		// This is meant to release only what the decoder uses internally, but not the
		// external buffers the application is working with!
		if (c == 0)
		{
			delete NativeDecoderOutput;
			NativeDecoderOutput = nullptr;
			delete this;
		}
		return c;
	}

	void SetIsValid(bool bInIsValid)
	{
		bIsValid = bInIsValid;
	}
	bool GetIsValid() const
	{
		return bIsValid;
	}

	void SetWidth(int32 InWidth)
	{
		Width = InWidth;
	}
	virtual int32 GetWidth() const override
	{
		return Width;
	}
	void SetHeight(int32 InHeight)
	{
		Height = InHeight;
	}
	virtual int32 GetHeight() const override
	{
		return Height;
	}
	virtual int64 GetPTS() const override
	{
		return PTS;
	}
	void SetPTS(int64 InPTS)
	{
		PTS = InPTS;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual const FVideoDecoderAllocFrameBufferResult* GetAllocatedBuffer() const
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return &Buffer;
	}

	void SetCropLeft(int32 InCrop)
	{
		CropLeft = InCrop;
	}
	virtual int32 GetCropLeft() const override
	{
		return CropLeft;
	}
	void SetCropRight(int32 InCrop)
	{
		CropRight = InCrop;
	}
	virtual int32 GetCropRight() const override
	{
		return CropRight;
	}
	void SetCropTop(int32 InCrop)
	{
		CropTop = InCrop;
	}
	virtual int32 GetCropTop() const override
	{
		return CropTop;
	}
	void SetCropBottom(int32 InCrop)
	{
		CropBottom = InCrop;
	}
	virtual int32 GetCropBottom() const override
	{
		return CropBottom;
	}
	void SetAspectX(int32 InAspect)
	{
		AspectX = InAspect;
	}
	virtual int32 GetAspectX() const override
	{
		return AspectX;
	}
	void SetAspectY(int32 InAspect)
	{
		AspectY = InAspect;
	}
	virtual int32 GetAspectY() const override
	{
		return AspectY;
	}
	void SetPitchX(int32 InPitch)
	{
		PitchX = InPitch;
	}
	virtual int32 GetPitchX() const override
	{
		return PitchX;
	}
	void SetPitchY(int32 InPitch)
	{
		PitchY = InPitch;
	}
	virtual int32 GetPitchY() const override
	{
		return PitchY;
	}
	void SetColorFormat(uint32 InColorFormat)
	{
		ColorFormat = InColorFormat;
	}
	virtual uint32 GetColorFormat() const override
	{
		return ColorFormat;
	}

	// Internal for allocation.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderAllocFrameBufferResult* GetBuffer()
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return &Buffer;
	}

	void SetNativeDecoderOutput(FNativeVideoDecoderOutput* InNativeDecoderOutput)
	{
		delete NativeDecoderOutput;
		NativeDecoderOutput = InNativeDecoderOutput;
	}

private:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderAllocFrameBufferResult	Buffer = {};
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	int32	RefCount = 1;
	int32	Width = 0;
	int32	Height = 0;
	int64	PTS = 0;

	int32	CropLeft = 0;
	int32	CropRight = 0;
	int32	CropTop = 0;
	int32	CropBottom = 0;
	int32	AspectX = 1;
	int32	AspectY = 1;
	int32	PitchX = 0;
	int32	PitchY = 0;
	uint32	ColorFormat = 0;

	FNativeVideoDecoderOutput* NativeDecoderOutput = nullptr;

	bool	bIsValid = true;
};



class FVideoDecoderH264_WindowsImpl : public FVideoDecoderH264_Windows
{
public:
	FVideoDecoderH264_WindowsImpl();

	virtual ~FVideoDecoderH264_WindowsImpl();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual bool Setup(const FInit& InInit) override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	virtual void Shutdown() override;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual EDecodeResult Decode(const FVideoDecoderInput* InInput) override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS



private:
	struct FInputToDecode
	{
		int64								PTS;
		int32								Width;
		int32								Height;
		TArray<uint8>						Data;
		bool								bIsKeyframe;
	};


	struct FDecoderOutputBuffer
	{
		FDecoderOutputBuffer()
		{
			OutputStreamInfo = {};
			OutputBuffer = {};
		}
		~FDecoderOutputBuffer()
		{
			UnprepareAfterProcess();
			if (OutputBuffer.pSample)
			{
				OutputBuffer.pSample->Release();
			}
		}
		TRefCountPtr<IMFSample> DetachOutputSample()
		{
			TRefCountPtr<IMFSample> pOutputSample;
			if (OutputBuffer.pSample)
			{
				pOutputSample = TRefCountPtr<IMFSample>(OutputBuffer.pSample, false);
				OutputBuffer.pSample = nullptr;
			}
			return pOutputSample;
		}
		void PrepareForProcess()
		{
			OutputBuffer.dwStatus   = 0;
			OutputBuffer.dwStreamID = 0;
			OutputBuffer.pEvents    = nullptr;
		}
		void UnprepareAfterProcess()
		{
			if (OutputBuffer.pEvents)
			{
				// https://docs.microsoft.com/en-us/windows/desktop/api/mftransform/nf-mftransform-imftransform-processoutput
				// The caller is responsible for releasing any events that the MFT allocates.
				OutputBuffer.pEvents->Release();
				OutputBuffer.pEvents = nullptr;
			}
		}
		MFT_OUTPUT_STREAM_INFO	OutputStreamInfo;
		MFT_OUTPUT_DATA_BUFFER	OutputBuffer;
	};


	static inline bool IsWindows8Plus()
	{ return true; }

	static bool StaticInitializeResources();
	static void StaticReleaseResources();

	bool CreateD3DResources();
	void ReleaseD3DResources();

	bool SetupFirstUseResources();

	bool CreateDecoderInstance();

	bool ProcessDecoding(FInputToDecode* InInput);
	bool CopyTexture(const TRefCountPtr<IMFSample>& DecodedOutputSample, FF5PlayerVideoDecoderOutputDX* DecoderOutput, FIntPoint OutputDim, FIntPoint NativeDim);
	bool ConvertDecodedImage(const TRefCountPtr<IMFSample>& DecodedOutputSample);

	void AllocateApplicationOutputSample();
	bool AllocateApplicationOutputSampleBuffer(int32 Width, int32 Height);
	void ReturnUnusedApplicationOutputSample();
	void PurgeAllPendingOutput();


	bool CreateDecoderOutputBuffer();


	void CreateWorkerThread();
	void StopWorkerThread();
	void WorkerThreadFN();
	void PerformAsyncShutdown();


	bool FallbackToSwDecoding(FString Reason);
	bool ReconfigureForSwDecoding(FString Reason);
	bool InternalDecoderCreate();
	bool Configure();
	bool StartStreaming();
	bool DecoderSetInputType();
	bool DecoderSetOutputType();
	bool DecoderVerifyStatus();
	void InternalDecoderDestroy();

	// DirectX device information
	struct FDXDeviceInfo
	{
		FDXDeviceInfo()	: DxVersion(ED3DVersion::VersionUnknown)
		{ }
		~FDXDeviceInfo()
		{
			Reset();
		}
		void Reset()
		{
			// Set to null in reverse order of creation to release the references.
			DxDevice = nullptr;
			DxDeviceContext = nullptr;
			DxDeviceManager = nullptr;
			DxVersion = ED3DVersion::VersionUnknown;
			DxMainAppDevice11 = nullptr;
			DxMainAppDevice12 = nullptr;
		}
		enum class ED3DVersion
		{
			VersionUnknown,
			Version9Win7,
			Version11Win8,
			Version11XB1,
			Version12Win10
		};

		ED3DVersion								DxVersion;
		TRefCountPtr<ID3D11Device>				DxDevice;
		TRefCountPtr<ID3D11DeviceContext>		DxDeviceContext;
		TRefCountPtr<IMFDXGIDeviceManager>		DxDeviceManager;
		TRefCountPtr<ID3D11Device>				DxMainAppDevice11;
		TRefCountPtr<ID3D12Device>				DxMainAppDevice12;
		/*
		Windows 7 legacy
			TRefCountPtr<IDirect3D9>				Dx9;
			TRefCountPtr<IDirect3DDevice9>			Dx9Device;
			TRefCountPtr<IDirect3DDeviceManager9>	Dx9DeviceManager;
		*/
	};


	enum class EDecodeMode
	{
		Undefined,
		//Win7,
		Win8HW,
		Win8SW,
		XB1,
		Win10HW,
		Win10SW
	};


	FDXDeviceInfo								DXDeviceInfo;
	bool										bIsInitialized;
	EDecodeMode									DecodeMode;

	TQueue<TUniquePtr<FInputToDecode>>			InputQueue;
	TQueue<FVideoDecoderOutputH264_Windows*>	OutputQueue;


	TUniquePtr<FThread>							WorkerThread;
	DecoderUtilities::FEventSignal				WorkerThreadSignalHaveWork;
	bool										bTerminateWorkerThread;

	TRefCountPtr<IMFTransform>					DecoderTransform;
	TRefCountPtr<IMFMediaType>					CurrentOutputMediaType;
	MFT_OUTPUT_STREAM_INFO						DecoderOutputStreamInfo;
	bool										bIsHardwareAccelerated;
	bool										bRequiresReconfigurationForSW;

	TUniquePtr<FDecoderOutputBuffer>			CurrentDecoderOutputBuffer;
	FVideoDecoderOutputH264_Windows*			CurrentApplicationOutputSample;
	FF5PlayerVideoDecoderOutputDX*				CurrentSoftwareDecoderOutput;
	int32										NumFramesInDecoder;

	static FCriticalSection						GlobalLock;
	static int32								GlobalInitCount;
	static bool									bGlobalMFStartedUp;

	static DWORD WINAPI AsyncShutdownProc(LPVOID lpParameter)
	{
		static_cast<FVideoDecoderH264_WindowsImpl*>(lpParameter)->PerformAsyncShutdown();
		return 0;
	}
};

FCriticalSection			FVideoDecoderH264_WindowsImpl::GlobalLock;
int32						FVideoDecoderH264_WindowsImpl::GlobalInitCount = 0;
bool						FVideoDecoderH264_WindowsImpl::bGlobalMFStartedUp = false;

/*********************************************************************************************************************/

/**
 * Registers this decoder with the decoder factory.
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FVideoDecoderH264_Windows::Register(FVideoDecoderFactory& InFactory)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderInfo	DecoderInfo;
	DecoderInfo.CodecType = ECodecType::H264;
	DecoderInfo.MaxWidth = 1920;
	DecoderInfo.MaxHeight = 1088;
	
	InFactory.Register(DecoderInfo, []() {
			return new FVideoDecoderH264_WindowsImpl();
		});
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

/*********************************************************************************************************************/

/**
 * Load the necessary DLLs and start up the Media Foundation (MF)
 */
bool FVideoDecoderH264_WindowsImpl::StaticInitializeResources()
{
	FScopeLock Lock(&GlobalLock);
	if (GlobalInitCount++ == 0)
	{
		// For the time being we require to run on Windows 8 and up.
		if (FWindowsPlatformMisc::VerifyWindowsVersion(6, 2))
		{
			// Those are the same for Win7, Win8 and Win10
			bool bOk = FPlatformProcess::GetDllHandle(TEXT("mf.dll"))
					&& FPlatformProcess::GetDllHandle(TEXT("mfplat.dll"))
					&& FPlatformProcess::GetDllHandle(TEXT("msmpeg2vdec.dll"));
			if (bOk)
			{
				// Start Media Foundation.
				HRESULT Result = MFStartup(MF_VERSION);
				if (SUCCEEDED(Result))
				{
					bGlobalMFStartedUp = true;
				}
				else
				{
					UE_LOGF(LogVideoDecoder, Error, "MFStartup() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
				}
			}
			else
			{
				UE_LOGF(LogVideoDecoder, Error, "Not all required DLLs were loaded");
			}
		}
		else
		{
			UE_LOGF(LogVideoDecoder, Error, "Windows versions less than Windows 8 are not supported");
		}
	}
	return bGlobalMFStartedUp;
}


/**
 * Close Media Foundation after the last instance is released.
 */
void FVideoDecoderH264_WindowsImpl::StaticReleaseResources()
{
	FScopeLock Lock(&GlobalLock);
	if (--GlobalInitCount == 0)
	{
		if (bGlobalMFStartedUp)
		{
			MFShutdown();
			bGlobalMFStartedUp = false;
		}
	}
}


/**
 * Create a D3D device to run the decoder transform on.
 */
bool FVideoDecoderH264_WindowsImpl::CreateD3DResources()
{
	// Ask the application for the D3D device and version it is using.
	void* ApplicationD3DDevice = nullptr;
	int32_t ApplicationD3DDeviceVersion = 0;
	bool bOk = false;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderMethodsWindows* AllocationMethods = reinterpret_cast<FVideoDecoderMethodsWindows*>(GetAllocationInterfaceMethods());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (AllocationMethods)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (AllocationMethods->MagicCookie == 0x57696e58) // 'WinX'
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			int32_t AppResult = AllocationMethods->GetD3DDevice.Execute(AllocationMethods->This, &ApplicationD3DDevice, &ApplicationD3DDeviceVersion);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (AppResult == 0)
			{
				// D3D version is returned as major*1000+minor.
				if (ApplicationD3DDeviceVersion >= 11000 && ApplicationD3DDeviceVersion <= 12999)
				{
					if (ApplicationD3DDevice)
					{
						bOk = true;
					}
					else
					{
						UE_LOGF(LogVideoDecoder, Error, "No application D3D device returned by video decoder allocation interface!");
					}
				}
				else
				{
					UE_LOGF(LogVideoDecoder, Error, "Unsupported D3D version returned (%d) which is neither DX11 or DX12.", ApplicationD3DDeviceVersion);
				}
			}
			else
			{
				UE_LOGF(LogVideoDecoder, Error, "Application video decoder allocation interface returned %d for GetD3DDevice()", AppResult);
			}
		}
		else
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			UE_LOGF(LogVideoDecoder, Error, "Incorrect video decoder allocation interface is set. Is %08x, should be %08x", AllocationMethods->MagicCookie, 0x57696e58);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
	else
	{
		UE_LOGF(LogVideoDecoder, Error, "No video decoder allocation interface is set");
	}
	// Leave if any problems found.
	if (!bOk)
	{
		return false;
	}

	// Reset the current device
	DXDeviceInfo.Reset();

	// Initialize our D3D decoder device.
	UINT ResetToken = 0;
	HRESULT Result = MFCreateDXGIDeviceManager(&ResetToken, DXDeviceInfo.DxDeviceManager.GetInitReference());
	if (SUCCEEDED(Result))
	{
		// We need to get access to the already existing D3D device to create a new device for the decoder from the same adapter.
		uint32 DxDeviceCreationFlags = 0;
		TRefCountPtr<IDXGIAdapter> DXGIAdapter;

		// D3D11
		if (ApplicationD3DDeviceVersion / 1000 == 11)
		{
			ID3D11Device* ApplicationDxDevice = static_cast<ID3D11Device*>(ApplicationD3DDevice);
			TRefCountPtr<IDXGIDevice> DXGIDevice;
			ApplicationDxDevice->QueryInterface(__uuidof(IDXGIDevice), (void**)DXGIDevice.GetInitReference());
			Result = DXGIDevice->GetAdapter((IDXGIAdapter**)DXGIAdapter.GetInitReference());
			if (SUCCEEDED(Result))
			{
				DxDeviceCreationFlags = ApplicationDxDevice->GetCreationFlags();
				DXDeviceInfo.DxVersion = FDXDeviceInfo::ED3DVersion::Version11Win8;
				DXDeviceInfo.DxMainAppDevice11 = TRefCountPtr<ID3D11Device>(ApplicationDxDevice);
			}
			else
			{
				UE_LOGF(LogVideoDecoder, Error, "ID3D11Device::GetAdapter() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			}
		}
		// D3D12
		else
		{
			TRefCountPtr<IDXGIFactory4> DXGIFactory;
			Result = CreateDXGIFactory(__uuidof(IDXGIFactory4), (void**)DXGIFactory.GetInitReference());
			if (SUCCEEDED(Result))
			{
				ID3D12Device* ApplicationDxDevice = static_cast<ID3D12Device*>(ApplicationD3DDevice);
				LUID Luid = ApplicationDxDevice->GetAdapterLuid();

				Result = DXGIFactory->EnumAdapterByLuid(Luid, __uuidof(IDXGIAdapter), (void**)DXGIAdapter.GetInitReference());
				if (SUCCEEDED(Result))
				{
					DXDeviceInfo.DxVersion = FDXDeviceInfo::ED3DVersion::Version12Win10;
					DXDeviceInfo.DxMainAppDevice12 = TRefCountPtr<ID3D12Device>(ApplicationDxDevice);
				}
				else
				{
					UE_LOGF(LogVideoDecoder, Error, "EnumAdapterByLuid() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
				}
			}
			else
			{
				UE_LOGF(LogVideoDecoder, Error, "CreateDXGIFactory() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			}
		}

		// Create the device only when we could successfully get the adapter. If we could not an error will have been logged already.
		if (SUCCEEDED(Result))
		{
			D3D_FEATURE_LEVEL FeatureLevel;

			uint32 DeviceCreationFlags = 0;
			if ((DxDeviceCreationFlags & D3D11_CREATE_DEVICE_DEBUG) != 0)
			{
				DeviceCreationFlags |= D3D11_CREATE_DEVICE_DEBUG;
			}

			Result = D3D11CreateDevice(DXGIAdapter, D3D_DRIVER_TYPE_UNKNOWN, nullptr, DeviceCreationFlags, nullptr, 0, D3D11_SDK_VERSION, DXDeviceInfo.DxDevice.GetInitReference(), &FeatureLevel, DXDeviceInfo.DxDeviceContext.GetInitReference());
			if (SUCCEEDED(Result))
			{
				if (FeatureLevel >= D3D_FEATURE_LEVEL_9_3)
				{
					Result = DXDeviceInfo.DxDeviceManager->ResetDevice(DXDeviceInfo.DxDevice, ResetToken);
					if (SUCCEEDED(Result))
					{
						// Multithread-protect the newly created device as we're going to use it from decoding thread and from render thread for texture
						// sharing between decoding and rendering DX devices.
						TRefCountPtr<ID3D10Multithread> DxMultithread;
						Result = DXDeviceInfo.DxDevice->QueryInterface(__uuidof(ID3D10Multithread), (void**)DxMultithread.GetInitReference());
						if (SUCCEEDED(Result))
						{
							DxMultithread->SetMultithreadProtected(1);
						}

						DXGI_ADAPTER_DESC AdapterDesc;
						DXGIAdapter->GetDesc(&AdapterDesc);
						UE_LOGF(LogVideoDecoder, Display, "Created D3D11 device for H.264 decoding on %ls.", *FString(AdapterDesc.Description));

						return true;
					}
					else
					{
						UE_LOGF(LogVideoDecoder, Error, "ResetDevice() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
					}
				}
				else
				{
					DXDeviceInfo.DxVersion = FDXDeviceInfo::ED3DVersion::VersionUnknown;
					UE_LOGF(LogVideoDecoder, Error, "Unable to Create D3D11 Device with feature level 9.3 or above");
				}
			}
			else
			{
				UE_LOGF(LogVideoDecoder, Error, "D3D11CreateDevice() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			}
		}
	}
	else
	{
		UE_LOGF(LogVideoDecoder, Error, "MFCreateDXGIDeviceManager() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
	}

	DXDeviceInfo.Reset();
	return false;
}


/**
 * Release the D3D device used by the decoder transform.
 */
void FVideoDecoderH264_WindowsImpl::ReleaseD3DResources()
{
	DXDeviceInfo.Reset();
}


/**
 * Reconfigures the decoder transform for software decoding.
 */
bool FVideoDecoderH264_WindowsImpl::FallbackToSwDecoding(FString Reason)
{
#if PLATFORM_WINDOWS
	if (!bIsHardwareAccelerated)
	{
		return false;
	}

	UE_LOGF(LogVideoDecoder, Display, "FallbackToSwDecoding: %ls", *Reason);

	bIsHardwareAccelerated = false;

	if (IsWindows8Plus())
	{
		// This may not be possible to do since that will mess with the global D3D device in a way the application may not expect!

		HRESULT Result = S_OK;
		TRefCountPtr<ID3D10Multithread> DxMultithread;
		if (DXDeviceInfo.DxMainAppDevice11.IsValid())
		{
			Result = DXDeviceInfo.DxMainAppDevice11->QueryInterface(__uuidof(ID3D10Multithread), (void**)DxMultithread.GetInitReference());
		}
		else if (DXDeviceInfo.DxMainAppDevice12.IsValid())
		{
			Result = DXDeviceInfo.DxMainAppDevice12->QueryInterface(__uuidof(ID3D10Multithread), (void**)DxMultithread.GetInitReference());
		}
		else
		{
			UE_LOGF(LogVideoDecoder, Error, "Failed to set video decoder into software mode (main D3D device multithread protect). No application D3D device set");
			return false;
		}
		if (SUCCEEDED(Result) && DxMultithread.IsValid())
		{
			DxMultithread->SetMultithreadProtected(1);
			return true;
		}
		else
		{
			UE_LOGF(LogVideoDecoder, Error, "Failed to set video decoder into software mode (main D3D device multithread protect) with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}
	}
#endif
	return false;
}


/**
 * Reconfigures the decoder transform for software decoding.
 */
bool FVideoDecoderH264_WindowsImpl::ReconfigureForSwDecoding(FString Reason)
{
	bRequiresReconfigurationForSW = true;
	if (!FallbackToSwDecoding(MoveTemp(Reason)))
	{
		return false;
	}

	// Nullify D3D Manager to switch decoder to software mode.
	if (DecoderTransform.GetReference())
	{
		HRESULT Result;
		Result = DecoderTransform->ProcessMessage(MFT_MESSAGE_SET_D3D_MANAGER, 0);
		if (SUCCEEDED(Result))
		{
			return true;
		}
		else
		{
			UE_LOGF(LogVideoDecoder, Error, "Failed to set video decoder into software mode (unset D3D manager) with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		}
	}
	return false;
}


/**
 * Creates a MFT decoder transform.
 */
bool FVideoDecoderH264_WindowsImpl::InternalDecoderCreate()
{
	TRefCountPtr<IMFAttributes>	Attributes;
	TRefCountPtr<IMFTransform>	Decoder;
	HRESULT					Result;

	if (Electra::IsWindows8Plus())
	{
		// Check if there is any reason for a "device lost" - if not we know all is stil well; otherwise we bail without creating a decoder
		Result = DXDeviceInfo.DxDevice->GetDeviceRemovedReason();
		if (Result != S_OK)
		{
			UE_LOGF(LogVideoDecoder, Error, "D3D device loss detected. Reason 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}
	}

	Result = CoCreateInstance(MFTmsH264Decoder, nullptr, CLSCTX_INPROC_SERVER, IID_IMFTransform, reinterpret_cast<void**>(&Decoder));
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "CoCreateInstance() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}
	Result = Decoder->GetAttributes(Attributes.GetInitReference());
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "GetAttributes() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}

#if PLATFORM_WINDOWS
	// Force SW decoding?
	if ((0))
	{
		FallbackToSwDecoding(FString::Printf(TEXT("Windows %s"), *FWindowsPlatformMisc::GetOSVersion()));
	}
	else
#endif
	{
		// Check if the transform is D3D aware
		if (IsWindows8Plus())	// true for XB1 as well
		{
			uint32 IsDX11Aware = 0;
			Result = Attributes->GetUINT32(MF_SA_D3D11_AWARE, &IsDX11Aware);
			if (FAILED(Result))
			{
				FallbackToSwDecoding(TEXT("Failed to get MF_SA_D3D11_AWARE"));
			}
			else if (IsDX11Aware == 0)
			{
				FallbackToSwDecoding(TEXT("Not MF_SA_D3D11_AWARE"));
			}
			else if (FAILED(Result = Decoder->ProcessMessage(MFT_MESSAGE_SET_D3D_MANAGER, reinterpret_cast<ULONG_PTR>(DXDeviceInfo.DxDeviceManager.GetReference()))))
			{
				FallbackToSwDecoding(FString::Printf(TEXT("Failed to set MFT_MESSAGE_SET_D3D_MANAGER: 0x%X %s"), (uint32)Result, *GetComErrorDescription(Result)));
			}
		}
	#if 0	// PLATFORM_WINDOWS
		// Not supported at the moment.
		else // Windows 7
		{
			if (!DXDeviceInfo->Dx9Device || !DXDeviceInfo->Dx9DeviceManager)
			{
				FallbackToSwDecoding(TEXT("Failed to create DirectX 9 device / device manager"));
			}

			uint32 IsD3DAware = 0;
			Result = Attributes->GetUINT32(MF_SA_D3D_AWARE, &IsD3DAware);
			if (FAILED(Result))
			{
				FallbackToSwDecoding(TEXT("Failed to get MF_SA_D3D_AWARE"));
			}
			else if (IsD3DAware == 0)
			{
				FallbackToSwDecoding(TEXT("Not MF_SA_D3D_AWARE"));
			}
			else if (FAILED(Result = Decoder->ProcessMessage(MFT_MESSAGE_SET_D3D_MANAGER, reinterpret_cast<ULONG_PTR>(DXDeviceInfo->Dx9DeviceManager.GetReference()))))
			{
				FallbackToSwDecoding(FString::Printf(TEXT("Failed to set MFT_MESSAGE_SET_D3D_MANAGER: 0x%08x %s"), Result, *GetComErrorDescription(Result)));
			}
		}
	#endif
	}

	// Try switching to low-latency mode.
	if (FAILED(Result = Attributes->SetUINT32(CODECAPI_AVLowLatencyMode, 1)))
	{
		// Not an error. If it doesn't work it just doesn't.
	}
	// Create successful, take on the decoder.
	DecoderTransform = Decoder;

	return true;
}


/**
 * Configures the decoder MFT
 */
bool FVideoDecoderH264_WindowsImpl::Configure()
{
	// Setup media input type
	if (!DecoderSetInputType())
	{
		return false;
	}
	if (bRequiresReconfigurationForSW)
	{
		return false;
	}

	// Setup media output type
	if (!DecoderSetOutputType())
	{
		return false;
	}
	if (bRequiresReconfigurationForSW)
	{
		return false;
	}

	// Verify status
	if (!DecoderVerifyStatus())
	{
		return false;
	}
	if (bRequiresReconfigurationForSW)
	{
		return false;
	}
	return true;
}


/**
 * Starts the decoder MFT.
 */
bool FVideoDecoderH264_WindowsImpl::StartStreaming()
{
	HRESULT Result;
	Result = DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_BEGIN_STREAMING, 0);
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "MFT_MESSAGE_NOTIFY_BEGIN_STREAMING failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}
	Result = DecoderTransform->ProcessMessage(MFT_MESSAGE_NOTIFY_START_OF_STREAM, 0);
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "MFT_MESSAGE_NOTIFY_START_OF_STREAM failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}
	return true;
}


/**
 * Sets the input type on the decoder MFT
 */
bool FVideoDecoderH264_WindowsImpl::DecoderSetInputType()
{
	TRefCountPtr<IMFMediaType>	InputMediaType;
	HRESULT						Result;

	// See https://docs.microsoft.com/en-us/windows/desktop/medfound/h-264-video-decoder
	Result = MFCreateMediaType(InputMediaType.GetInitReference());
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "MFCreateMediaType() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}
	Result = InputMediaType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Video);
	if (SUCCEEDED(Result))
	{
		Result = InputMediaType->SetGUID(MF_MT_SUBTYPE, MFVideoFormat_H264);
	}
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "Failed to set input media type with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}

	// Set to FullHD resolution. This is the largest we can rely on to be supported in hardware.
	int32 configW, configH;
	configW = 1920;
	configH = 1088;

	Result = MFSetAttributeSize(InputMediaType, MF_MT_FRAME_SIZE, configW, configH);
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "Failed to set video decoder input media type resolution of %d*%d with 0x%08x (%ls)", configW, configH, Result, *GetComErrorDescription(Result));
		return false;
	}

	Result = DecoderTransform->SetInputType(0, InputMediaType, 0);

#if PLATFORM_WINDOWS
	if (bIsHardwareAccelerated && Result == MF_E_UNSUPPORTED_D3D_TYPE)
		// h/w acceleration is not supported, e.g. unsupported resolution (4K), fall back to s/w decoding
	{
		return ReconfigureForSwDecoding(TEXT("MF_E_UNSUPPORTED_D3D_TYPE"));
	}
	else
#endif
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "SetInputType() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}

	return true;
}


/**
 * Sets the desired output format on the decoder MFT.
 */
bool FVideoDecoderH264_WindowsImpl::DecoderSetOutputType()
{
	TRefCountPtr<IMFMediaType>	OutputMediaType;
	GUID						OutputMediaMajorType;
	GUID						OutputMediaSubtype;
	HRESULT						Result;

	// Supposedly calling GetOutputAvailableType() returns following output media subtypes:
	// MFVideoFormat_NV12, MFVideoFormat_YV12, MFVideoFormat_IYUV, MFVideoFormat_I420, MFVideoFormat_YUY2
	for(int32 TypeIndex=0; ; ++TypeIndex)
	{
		Result = DecoderTransform->GetOutputAvailableType(0, TypeIndex, OutputMediaType.GetInitReference());
		if (FAILED(Result))
		{
			UE_LOGF(LogVideoDecoder, Error, "GetOutputAvailableType() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}
		Result = OutputMediaType->GetGUID(MF_MT_MAJOR_TYPE, &OutputMediaMajorType);
		if (SUCCEEDED(Result))
		{
			Result = OutputMediaType->GetGUID(MF_MT_SUBTYPE, &OutputMediaSubtype);
		}
		if (FAILED(Result))
		{
			UE_LOGF(LogVideoDecoder, Error, "Failed to get video decoder available output media (sub-)type with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}
		if (OutputMediaMajorType == MFMediaType_Video && OutputMediaSubtype == MFVideoFormat_NV12)
		{
			Result = DecoderTransform->SetOutputType(0, OutputMediaType, 0);
			if (SUCCEEDED(Result))
			{
				CurrentOutputMediaType = OutputMediaType;
				return true;
			}
			else
			{
				UE_LOGF(LogVideoDecoder, Error, "SetOutputType() failed  with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
				return false;
			}
		}
	}
}


/**
 * Verifies that the decoder MFT is in the state we expect it to be in.
 */
bool FVideoDecoderH264_WindowsImpl::DecoderVerifyStatus()
{
	HRESULT		Result;
	DWORD		NumInputStreams;
	DWORD		NumOutputStreams;

	Result = DecoderTransform->GetStreamCount(&NumInputStreams, &NumOutputStreams);
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "GetStreamCount() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}
	if (NumInputStreams != 1 || NumOutputStreams != 1)
	{
		UE_LOGF(LogVideoDecoder, Error, "Unexpected number of streams: input %d, output %d", (int32)NumInputStreams, (int32)NumOutputStreams);
		return false;
	}

	DWORD DecoderStatus = 0;
	Result = DecoderTransform->GetInputStatus(0, &DecoderStatus);
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "GetInputStatus() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}
	if (MFT_INPUT_STATUS_ACCEPT_DATA != DecoderStatus)
	{
		UE_LOGF(LogVideoDecoder, Error, "Decoder doesn't accept data, status %d", (int32)DecoderStatus);
		return false;
	}

	Result = DecoderTransform->GetOutputStreamInfo(0, &DecoderOutputStreamInfo);
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "GetOutputStreamInfo() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}
	if (!(DecoderOutputStreamInfo.dwFlags & MFT_OUTPUT_STREAM_FIXED_SAMPLE_SIZE))
	{
		return ReconfigureForSwDecoding(TEXT("Incompatible H.264 decoder: Fixed sample size expected"));
	}
	if (!(DecoderOutputStreamInfo.dwFlags & MFT_OUTPUT_STREAM_WHOLE_SAMPLES))
	{
		return ReconfigureForSwDecoding(TEXT("Incompatible H.264 decoder: Whole samples expected"));
	}
	if (bIsHardwareAccelerated && !(DecoderOutputStreamInfo.dwFlags & MFT_OUTPUT_STREAM_PROVIDES_SAMPLES))
	{
		return ReconfigureForSwDecoding(TEXT("Incompatible H.264 decoder: H/W accelerated decoder is expected to provide output samples"));
	}
	if (!bIsHardwareAccelerated && (DecoderOutputStreamInfo.dwFlags & MFT_OUTPUT_STREAM_PROVIDES_SAMPLES))
	{
		UE_LOGF(LogVideoDecoder, Error, "Incompatible H.264 decoder: s/w decoder is expected to require preallocated output samples");
		return false;
	}
	return true;
}

/*********************************************************************************************************************/
/*********************************************************************************************************************/
/*********************************************************************************************************************/

/**
* Destroys the current decoder instance.
*/
void FVideoDecoderH264_WindowsImpl::InternalDecoderDestroy()
{
	DecodeMode = EDecodeMode::Undefined;
	DecoderTransform = nullptr;
	CurrentOutputMediaType = nullptr;
	NumFramesInDecoder = 0;
	DecoderOutputStreamInfo = {};
	bIsHardwareAccelerated = true;
	bRequiresReconfigurationForSW = false;
	CurrentDecoderOutputBuffer.Reset();
}


/**
 * Create, configure and start a decoder transform.
 */
bool FVideoDecoderH264_WindowsImpl::CreateDecoderInstance()
{
	NumFramesInDecoder = 0;
	bool bOk = true;
	// Create a decoder transform. This will determine if we will be using a hardware or software decoder.
	// Start out assuming it will be a hardware accelerated decoder.
	bIsHardwareAccelerated = true;
	if (InternalDecoderCreate())
	{
		// Configure the decoder with our default values.
		bRequiresReconfigurationForSW = false;
		Configure();
		if (bRequiresReconfigurationForSW)
		{
			// No failure yet, but a switch to software decoding is required. We do the configuration over one more time.
			check(!bIsHardwareAccelerated);	// must have been reset already!
			// Clear this out, we can't get another request for reconfiguration since we already did.
			bRequiresReconfigurationForSW = false;
			Configure();
			// If reconfiguration is still required we failed to create a software decoder.
			if (bRequiresReconfigurationForSW)
			{
				UE_LOGF(LogVideoDecoder, Error, "Failed to switch H.264 decoder transform to software decoding");
				bOk = false;
			}
		}
		
		// Set up the type of decode mode
		switch(DXDeviceInfo.DxVersion)
		{
			default:
			{
				UE_LOGF(LogVideoDecoder, Error, "Unsupported version of DirectX");
				DecodeMode = EDecodeMode::Undefined;
				bOk = false;
				break;
			}
			case FDXDeviceInfo::ED3DVersion::Version9Win7:
			{
				UE_LOGF(LogVideoDecoder, Error, "DirectX9 is not supported!");
				DecodeMode = EDecodeMode::Undefined;
				bOk = false;
				break;
			}
			case FDXDeviceInfo::ED3DVersion::Version11Win8:
			{
				DecodeMode = bIsHardwareAccelerated ? EDecodeMode::Win8HW : EDecodeMode::Win8SW;
				break;
			}
			case FDXDeviceInfo::ED3DVersion::Version11XB1:
			{
				DecodeMode = EDecodeMode::XB1;
				break;
			}
			case FDXDeviceInfo::ED3DVersion::Version12Win10:
			{
				DecodeMode = bIsHardwareAccelerated ? EDecodeMode::Win10HW : EDecodeMode::Win10SW;
				break;
			}
		}

		if (bOk)
		{
			bOk = StartStreaming();
		}
		// On failure clean up after ourselves.
		if (!bOk)
		{
			InternalDecoderDestroy();
		}
	}
	else
	{
		// Could not create decoder. The reason should have been logged already.
		bOk = false;
	}
	return bOk;
}


void FVideoDecoderH264_WindowsImpl::CreateWorkerThread()
{
	// Create and start the worker thread.
	bTerminateWorkerThread = false;
	WorkerThread = MakeUnique<FThread>(TEXT("AVEncoder::WindowsH264DecoderMFT"), [this]() { WorkerThreadFN(); });
}

void FVideoDecoderH264_WindowsImpl::StopWorkerThread()
{
	bTerminateWorkerThread = true;
	WorkerThreadSignalHaveWork.Signal();
	WorkerThread->Join();
	WorkerThread.Reset();
}


void FVideoDecoderH264_WindowsImpl::AllocateApplicationOutputSample()
{
	if (!CurrentApplicationOutputSample)
	{
		CurrentApplicationOutputSample = new FVideoDecoderOutputH264_Windows;
	}
}

bool FVideoDecoderH264_WindowsImpl::AllocateApplicationOutputSampleBuffer(int32 Width, int32 Height)
{
	check(CurrentApplicationOutputSample);

	// Set initial width and height. This could be changed later if for allocation purposes 
	// different (larger) values than the actual resolution are required.
	CurrentApplicationOutputSample->SetWidth(Width);
	CurrentApplicationOutputSample->SetHeight(Height);

	// Get the buffer for the output sample from the application. This is the part that gets wrapped in a
	// webrtc structure and passed back out to the application.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoderAllocFrameBufferParams ap {};
	EFrameBufferAllocReturn ar;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (DecodeMode == EDecodeMode::Win8HW)
	{
		// We will pass a ID3D11Texture2D pointer to a texture of _our_ decoding device to _share_ with the renderer.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ap.FrameBufferType = EFrameBufferType::CODEC_TextureHandle;
		ap.AllocSize = sizeof(ID3D11Texture2D*);
		ap.AllocAlignment = alignof(ID3D11Texture2D*);
		ap.AllocFlags = 0;
		// Set width and height for reference only.
		ap.Width = Width;
		ap.Height = Height;
		ap.BytesPerPixel = 1;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else if (DecodeMode == EDecodeMode::Win10HW || DecodeMode == EDecodeMode::Win10SW || DecodeMode == EDecodeMode::XB1)
	{
		// We need a byte buffer to put the image into. On the application side this may be realized as a TArray<uint8>.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ap.FrameBufferType = EFrameBufferType::CODEC_RawBuffer;
		ap.AllocSize = Width * Height * 3 / 2;
		ap.AllocAlignment = 1;
		ap.AllocFlags = 0;
		ap.Width = Width;
		ap.Height = Height;
		ap.BytesPerPixel = 1;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else if (DecodeMode == EDecodeMode::Win8SW)
	{
		// We want to have an ID3D11Texture2D we can give the decoder to decode into.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ap.FrameBufferType = EFrameBufferType::CODEC_TextureObject;
		ap.AllocSize = sizeof(ID3D11Texture2D*);
		ap.AllocAlignment = alignof(ID3D11Texture2D*);
		ap.AllocFlags = 0;
		ap.Width = Width;
		ap.Height = Height;
		ap.BytesPerPixel = 1;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		UE_LOGF(LogVideoDecoder, Error, "Unsupported decoding mode");
		return false;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	ar = AllocateOutputFrameBuffer(CurrentApplicationOutputSample->GetBuffer(), &ap);
	if (ar == EFrameBufferAllocReturn::CODEC_Success)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		// Got an output buffer.
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		check(CurrentApplicationOutputSample->GetBuffer()->AllocatedBuffer);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return true;
	}
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	else if (ar == EFrameBufferAllocReturn::CODEC_TryAgainLater)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		UE_LOGF(LogVideoDecoder, Error, "Unsupported allocation return value");
		return false;
	}
	else
	{
		// Error!
		return false;
	}
}

void FVideoDecoderH264_WindowsImpl::ReturnUnusedApplicationOutputSample()
{
	if (CurrentApplicationOutputSample)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (FVideoDecoderAllocFrameBufferResult* afb = CurrentApplicationOutputSample->GetBuffer())
		{
			afb->ReleaseCallback.ExecuteIfBound(afb->CallbackValue, afb->AllocatedBuffer);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		CurrentApplicationOutputSample->Release();
		CurrentApplicationOutputSample = nullptr;
	}
	delete CurrentSoftwareDecoderOutput;
	CurrentSoftwareDecoderOutput = nullptr;
}


bool FVideoDecoderH264_WindowsImpl::CreateDecoderOutputBuffer()
{
	TUniquePtr<FDecoderOutputBuffer>	NewDecoderOutputBuffer(new FDecoderOutputBuffer());
	HRESULT								Result;

	VERIFY_HR(DecoderTransform->GetOutputStreamInfo(0, &NewDecoderOutputBuffer->OutputStreamInfo), TEXT("Failed to get video decoder output stream information"));
	// Do we need to provide the sample output buffer or does the decoder create it for us?
	if ((NewDecoderOutputBuffer->OutputStreamInfo.dwFlags & (MFT_OUTPUT_STREAM_PROVIDES_SAMPLES | MFT_OUTPUT_STREAM_CAN_PROVIDE_SAMPLES)) == 0)
	{
		check(bIsHardwareAccelerated == false);
#if PLATFORM_WINDOWS
		check(CurrentSoftwareDecoderOutput);
		if (CurrentSoftwareDecoderOutput)
		{
			NewDecoderOutputBuffer->OutputBuffer.pSample = CurrentSoftwareDecoderOutput->GetMFSample().GetReference();
			if (NewDecoderOutputBuffer->OutputBuffer.pSample)
			{
				// Destination is a plain old pointer, we need to ref manually
				NewDecoderOutputBuffer->OutputBuffer.pSample->AddRef();
			}
		}
		else
#endif
		{
			return false;
		}
	}
	CurrentDecoderOutputBuffer = MoveTemp(NewDecoderOutputBuffer);
	return true;
}


bool FVideoDecoderH264_WindowsImpl::ProcessDecoding(FInputToDecode* InInput)
{
	if (DecoderTransform.GetReference() && InInput)
	{
		// Create the input sample.
		TRefCountPtr<IMFSample>			InputSample;
		TRefCountPtr<IMFMediaBuffer>	InputSampleBuffer;
		BYTE*							pbNewBuffer = nullptr;
		DWORD							dwMaxBufferSize = 0;
		DWORD							dwSize = 0;
		LONGLONG						llSampleTime = 0;
		HRESULT							Result;

		VERIFY_HR(MFCreateSample(InputSample.GetInitReference()), TEXT("Failed to create video decoder input sample"));
		VERIFY_HR(MFCreateMemoryBuffer((DWORD)InInput->Data.Num(), InputSampleBuffer.GetInitReference()), TEXT("Failed to create video decoder input sample memory buffer"));
		VERIFY_HR(InputSample->AddBuffer(InputSampleBuffer.GetReference()), TEXT("Failed to set video decoder input buffer with sample"));
		VERIFY_HR(InputSampleBuffer->Lock(&pbNewBuffer, &dwMaxBufferSize, &dwSize), TEXT("Failed to lock video decoder input sample buffer"));
		FMemory::Memcpy(pbNewBuffer, InInput->Data.GetData(), InInput->Data.Num());
		VERIFY_HR(InputSampleBuffer->Unlock(), TEXT("Failed to unlock video decoder input sample buffer"));
		VERIFY_HR(InputSampleBuffer->SetCurrentLength((DWORD) InInput->Data.Num()), TEXT("Failed to set video decoder input sample buffer length"));
		// Set sample attributes
		llSampleTime = InInput->PTS;
		VERIFY_HR(InputSample->SetSampleTime(llSampleTime), TEXT("Failed to set video decoder input sample presentation time"));
		VERIFY_HR(InputSample->SetUINT32(MFSampleExtension_CleanPoint, InInput->bIsKeyframe ? 1 : 0), TEXT("Failed to set video decoder input sample clean point"));

		// The input dimensions are the active pixels, but H.264 operates on 16x16 pixel macroblocks and we need buffers accommodating the encoded resolution.
		int32 InWidth  = (InInput->Width + 15) & ~15;
		int32 InHeight = (InInput->Height + 15) & ~15;

		// Loop until the decoder has consumed the input
		while(!bTerminateWorkerThread && InputSample.IsValid())
		{
			// Get a new output sample (just the wrapper, not the data buffer).
			AllocateApplicationOutputSample();
			// In software decode modes we need to get an output buffer for the output sample before we can call ProcessOutput().
			if (DecodeMode == EDecodeMode::Win8SW || DecodeMode == EDecodeMode::Win10SW)
			{
				// DX11 software mode needs a texture prepared by the application
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (DecodeMode == EDecodeMode::Win8SW && !CurrentApplicationOutputSample->GetBuffer()->AllocatedBuffer)
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				{
					if (!AllocateApplicationOutputSampleBuffer(InWidth, InHeight))
					{
						return false;
					}
				}
				if (!CurrentSoftwareDecoderOutput)
				{
					CurrentSoftwareDecoderOutput = new FF5PlayerVideoDecoderOutputDX;
				}
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (!CurrentSoftwareDecoderOutput->PreInitForSoftwareDecode(FIntPoint(InWidth, InHeight), DecodeMode == EDecodeMode::Win8SW ? *static_cast<ID3D11Texture2D**>(CurrentApplicationOutputSample->GetBuffer()->AllocatedBuffer) : nullptr))
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				{
					return false;
				}
			}

			// Prepare the output buffer the decoder will put the result into.
			if (!CurrentDecoderOutputBuffer.Get())
			{
				if (!CreateDecoderOutputBuffer())
				{
					return false;
				}
			}

			CurrentDecoderOutputBuffer->PrepareForProcess();
			DWORD	dwStatus = 0;
			Result = DecoderTransform->ProcessOutput(0, 1, &CurrentDecoderOutputBuffer->OutputBuffer, &dwStatus);
			CurrentDecoderOutputBuffer->UnprepareAfterProcess();

			if (Result == MF_E_TRANSFORM_NEED_MORE_INPUT)
			{
				if (InputSample.IsValid())
				{

					VERIFY_HR(DecoderTransform->ProcessInput(0, InputSample.GetReference(), 0), TEXT("Failed to process video decoder input"));
					// Used this sample. Have no further input data for now, but continue processing to produce output if possible.
					InputSample = nullptr;
					++NumFramesInDecoder;
				}
				else
				{
					// Need more input but have none right now.
					return true;
				}
			}
			else if (Result == MF_E_TRANSFORM_STREAM_CHANGE)
			{
				// Update output type.
				if (!DecoderSetOutputType())
				{
					return false;
				}
			}
			else if (SUCCEEDED(Result))
			{
				TRefCountPtr<IMFSample> DecodedOutputSample = CurrentDecoderOutputBuffer->DetachOutputSample();
				CurrentDecoderOutputBuffer.Reset();
				--NumFramesInDecoder;
				if (DecodedOutputSample)
				{
					if (ConvertDecodedImage(DecodedOutputSample))
					{
						OutputQueue.Enqueue(CurrentApplicationOutputSample);
						CurrentApplicationOutputSample = nullptr;
					}
					else
					{
						return false;
					}
				}
			}
			else
			{
				if (FAILED(Result))
				{
					UE_LOGF(LogVideoDecoder, Error, "Failed to process video decoder output with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
				}
				return false;
			}
		}
		return true;
	}
	return false;
}



bool FVideoDecoderH264_WindowsImpl::CopyTexture(const TRefCountPtr<IMFSample>& DecodedOutputSample, FF5PlayerVideoDecoderOutputDX* DecoderOutput, FIntPoint OutputDim, FIntPoint NativeDim)
{
#if PLATFORM_WINDOWS
	HRESULT Result;
	DWORD BuffersNum = 0;
	Result = DecodedOutputSample->GetBufferCount(&BuffersNum);
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "IMFSample::GetBufferCount() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}
	else if (BuffersNum != 1)
	{
		UE_LOGF(LogVideoDecoder, Error, "IMFSample::GetBufferCount() returned %d buffers instead of 1", (int32)BuffersNum);
		return false;
	}

	TRefCountPtr<IMFMediaBuffer> Buffer;
	Result = DecodedOutputSample->GetBufferByIndex(0, Buffer.GetInitReference());
	if (FAILED(Result))
	{
		UE_LOGF(LogVideoDecoder, Error, "IMFSample::GetBufferByIndex() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
		return false;
	}

	if (DecodeMode == EDecodeMode::Win8HW)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		check(CurrentApplicationOutputSample->GetBuffer()->AllocatedBuffer == nullptr);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (!AllocateApplicationOutputSampleBuffer(OutputDim.X, OutputDim.Y))
		{
			UE_LOGF(LogVideoDecoder, Error, "Failed to get a decode output buffer from the application!");
			return false;
		}
		DecoderOutput->InitializeWithSharedTexture(DXDeviceInfo.DxDevice, DXDeviceInfo.DxDeviceContext, DecodedOutputSample, OutputDim);
	}
	else if (DecodeMode == EDecodeMode::Win8SW)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		check(CurrentApplicationOutputSample->GetBuffer()->AllocatedBuffer != nullptr);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		DecoderOutput->InitializeWithTextureBuffer(OutputDim.X, FIntPoint(OutputDim.X, OutputDim.Y));// * 3 / 2));
	}
	else if (DecodeMode == EDecodeMode::Win10SW)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		check(CurrentApplicationOutputSample->GetBuffer()->AllocatedBuffer == nullptr);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (!AllocateApplicationOutputSampleBuffer(NativeDim.X, NativeDim.Y))
		{
			UE_LOGF(LogVideoDecoder, Error, "Failed to get a decode output buffer from the application!");
			return false;
		}

		DWORD BufferSize = 0;
		Result = Buffer->GetCurrentLength(&BufferSize);
		if (FAILED(Result))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFMediaBuffer::GetCurrentLength() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}

		uint8* Data = nullptr;
		Result = Buffer->Lock(&Data, NULL, NULL);
		if (FAILED(Result))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFMediaBuffer::Lock() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}

		DecoderOutput->InitializeWithBuffer(OutputDim.X, FIntPoint(OutputDim.X, OutputDim.Y * 3 / 2));

		// Copy NV12 texture data into external application buffer
		check(CurrentApplicationOutputSample);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		check((DWORD)CurrentApplicationOutputSample->GetBuffer()->AllocatedSize >= BufferSize);
		FMemory::Memcpy(CurrentApplicationOutputSample->GetBuffer()->AllocatedBuffer, Data, BufferSize);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		Result = Buffer->Unlock();
		if (FAILED(Result))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFMediaBuffer::Unlock() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}

		return true;
	}
	else if (DecodeMode == EDecodeMode::Win10HW)
	{
		TRefCountPtr<IMFDXGIBuffer> DXGIBuffer;
		Result = Buffer->QueryInterface(__uuidof(IMFDXGIBuffer), (void**)DXGIBuffer.GetInitReference());
		if (FAILED(Result))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFMediaBuffer::QueryInterface(IMFDXGIBuffer) failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}
		TRefCountPtr<ID3D11Texture2D> Texture2D;
		Result = DXGIBuffer->GetResource(IID_PPV_ARGS(Texture2D.GetInitReference()));
		if (FAILED(Result))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFDXGIBuffer::QueryInterface(ID3D11Texture2D) failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}
		D3D11_TEXTURE2D_DESC TextureDesc;
		Texture2D->GetDesc(&TextureDesc);
		if (TextureDesc.Format != DXGI_FORMAT_NV12)
		{
			UE_LOGF(LogVideoDecoder, Error, "FVideoDecoderH264_WindowsImpl::CopyTexture(): Decoded texture is not in NV12 format");
			return false;
		}

		TRefCountPtr<IMF2DBuffer> Buffer2D;
		Result = Buffer->QueryInterface(__uuidof(IMF2DBuffer), (void**)Buffer2D.GetInitReference());
		if (FAILED(Result))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFDXGIBuffer::QueryInterface(IMF2DBuffer) failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}

		uint8* Data = nullptr;
		LONG Pitch;
		Result = Buffer2D->Lock2D(&Data, &Pitch);
		if (FAILED(Result))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFMediaBuffer::Lock2D() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		check(CurrentApplicationOutputSample->GetBuffer()->AllocatedBuffer == nullptr);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		if (!AllocateApplicationOutputSampleBuffer(Pitch, TextureDesc.Height))
		{
			UE_LOGF(LogVideoDecoder, Error, "Failed to get a decode output buffer from the application!");
			return false;
		}
		CurrentApplicationOutputSample->SetWidth(OutputDim.X);
		//CurrentApplicationOutputSample->SetHeight(OutputDim.Y);
		CurrentApplicationOutputSample->SetHeight(TextureDesc.Height);
		CurrentApplicationOutputSample->SetPitchX(Pitch);

		DWORD BufferSize = Pitch * (TextureDesc.Height * 3 / 2);
		DecoderOutput->InitializeWithBuffer(Pitch, FIntPoint(TextureDesc.Width, TextureDesc.Height * 3 / 2));

		// Copy NV12 texture data into external application buffer
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		check((DWORD)CurrentApplicationOutputSample->GetBuffer()->AllocatedSize >= BufferSize);
		FMemory::Memcpy(CurrentApplicationOutputSample->GetBuffer()->AllocatedBuffer, Data, BufferSize);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		Result = Buffer2D->Unlock2D();
		if (FAILED(Result))
		{
			UE_LOGF(LogVideoDecoder, Error, "IMFMediaBuffer::Unlock2D() failed with 0x%08x (%ls)", Result, *GetComErrorDescription(Result));
			return false;
		}

		return true;
	}
	else
	{
		UE_LOGF(LogVideoDecoder, Error, "FVideoDecoderH264::CopyTexture(): Unhandled D3D version");
		return false;
	}
#endif

	return true;
}


bool FVideoDecoderH264_WindowsImpl::ConvertDecodedImage(const TRefCountPtr<IMFSample>& DecodedOutputSample)
{
	HRESULT		Result;
	LONGLONG	llTimeStamp = 0;
	MFVideoArea	videoArea {};
	UINT32		uiPanScanEnabled = 0;
	UINT32		dwInputWidth = 0;
	UINT32		dwInputHeight = 0;
	UINT32		stride = 0;
	UINT32		num=0, denom=0;

	// Get dimensions first. We need that to get the output buffer from the application.
	if (SUCCEEDED(Result = CurrentOutputMediaType->GetUINT32(MF_MT_PAN_SCAN_ENABLED, &uiPanScanEnabled)) && uiPanScanEnabled)
	{
		Result = CurrentOutputMediaType->GetBlob(MF_MT_PAN_SCAN_APERTURE, (UINT8*)&videoArea, sizeof(MFVideoArea), nullptr);
	}
	Result = MFGetAttributeSize(CurrentOutputMediaType.GetReference(), MF_MT_FRAME_SIZE, &dwInputWidth, &dwInputHeight);
	check(SUCCEEDED(Result));
	Result = CurrentOutputMediaType->GetBlob(MF_MT_MINIMUM_DISPLAY_APERTURE, (UINT8*)&videoArea, sizeof(MFVideoArea), nullptr);
	if (FAILED(Result))
	{
		Result = CurrentOutputMediaType->GetBlob(MF_MT_GEOMETRIC_APERTURE, (UINT8*)&videoArea, sizeof(MFVideoArea), nullptr);
	}

	check(CurrentApplicationOutputSample);
	CurrentApplicationOutputSample->SetCropLeft(0);
	CurrentApplicationOutputSample->SetCropTop(0);
	CurrentApplicationOutputSample->SetCropRight(dwInputWidth - videoArea.Area.cx);
	CurrentApplicationOutputSample->SetCropBottom(dwInputHeight - videoArea.Area.cy);

	// Try to get the stride. Defaults to 0 should it not be obtainable.
	stride = MFGetAttributeUINT32(CurrentOutputMediaType, MF_MT_DEFAULT_STRIDE, 0);
	CurrentApplicationOutputSample->SetPitchX(stride);

	// Try to get the pixel aspect ratio
	num=0, denom=0;
	MFGetAttributeRatio(CurrentOutputMediaType, MF_MT_PIXEL_ASPECT_RATIO, &num, &denom);
	if (!num || !denom)
	{
		num   = 1;
		denom = 1;
	}
	CurrentApplicationOutputSample->SetAspectX(num);
	CurrentApplicationOutputSample->SetAspectY(denom);

	VERIFY_HR(DecodedOutputSample->GetSampleTime(&llTimeStamp), TEXT("Failed to get video decoder output sample timestamp"));
	CurrentApplicationOutputSample->SetPTS((int64) llTimeStamp);

	FF5PlayerVideoDecoderOutputDX* DecoderOutput = new FF5PlayerVideoDecoderOutputDX;
	CopyTexture(DecodedOutputSample, DecoderOutput, FIntPoint(videoArea.Area.cx, videoArea.Area.cy), FIntPoint(dwInputWidth, dwInputHeight));
	CurrentApplicationOutputSample->SetNativeDecoderOutput(DecoderOutput);
	if (DecodeMode == EDecodeMode::Win8HW)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ID3D11Texture2D** ApplicationTexturePtr = reinterpret_cast<ID3D11Texture2D**>(CurrentApplicationOutputSample->GetBuffer()->AllocatedBuffer);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		// We set the texture in the user provided buffer. Note that we do NOT add the ref count here. This is ok because we have the
		// DecoderOutput stored in CurrentApplicationOutputSample where it is held. If the user application code uses the texture it
		// will/shall attach it to a ref counter pointer to retain it.
		*ApplicationTexturePtr = DecoderOutput->GetTexture().GetReference();
	}
	delete CurrentSoftwareDecoderOutput;
	CurrentSoftwareDecoderOutput = nullptr;

	switch(DecoderOutput->GetOutputType())
	{
		default:
			CurrentApplicationOutputSample->SetColorFormat(0);
			break;
		case FVideoDecoderOutputDX::EOutputType::SoftwareWin7:
			CurrentApplicationOutputSample->SetColorFormat(1);
			break;
		case FVideoDecoderOutputDX::EOutputType::SoftwareWin8Plus:
			CurrentApplicationOutputSample->SetColorFormat(2);
			break;
		case FVideoDecoderOutputDX::EOutputType::HardwareWin8Plus:
			CurrentApplicationOutputSample->SetColorFormat(3);
			break;
		case FVideoDecoderOutputDX::EOutputType::HardwareDX9_DX12:
			CurrentApplicationOutputSample->SetColorFormat(4);
			break;
	}
	return true;
}



void FVideoDecoderH264_WindowsImpl::WorkerThreadFN()
{
	while(!bTerminateWorkerThread)
	{
		WorkerThreadSignalHaveWork.WaitAndReset();
		if (bTerminateWorkerThread)
		{
			break;
		}

		// Handle all enqueued tasks when waking up.
		TUniquePtr<FInputToDecode> Input;
		while(!bTerminateWorkerThread && InputQueue.Dequeue(Input))
		{
			bool bOk = false;
			// Create a new decoder if necessary.
			if (!DecoderTransform.GetReference())
			{
				if (Input->bIsKeyframe)
				{
					bOk = CreateDecoderInstance();
					check(bOk);
				}
			}

			// Decode the sample.
			if (!bTerminateWorkerThread && DecoderTransform.GetReference())
			{
				bOk = ProcessDecoding(Input.Get());
				if (!bOk)
				{
					InternalDecoderDestroy();
					PurgeAllPendingOutput();
					InputQueue.Empty();
					FVideoDecoderOutputH264_Windows* FailedOutput = new FVideoDecoderOutputH264_Windows;
					FailedOutput->SetIsValid(false);
					OutputQueue.Enqueue(FailedOutput);
				}
			}
		}
		Input.Reset();
	}

	// Destroy the decoder transform.
	InternalDecoderDestroy();
	// Purge all output that was not delivered
	PurgeAllPendingOutput();
	// Any unprocessed input we release as well.
	InputQueue.Empty();
	bTerminateWorkerThread = false;
}


void FVideoDecoderH264_WindowsImpl::PurgeAllPendingOutput()
{
	ReturnUnusedApplicationOutputSample();
	FVideoDecoderOutputH264_Windows* Output = nullptr;
	while(OutputQueue.Dequeue(Output))
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (FVideoDecoderAllocFrameBufferResult* afb = Output->GetBuffer())
		{
			afb->ReleaseCallback.ExecuteIfBound(afb->CallbackValue, afb->AllocatedBuffer);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		Output->Release();
		Output = nullptr;
	}
}




/**
 * Constructor. Should only initialize member variables but not create a
 * decoder or start any worker threads. This may be called several times
 * over without actually decoding something.
 * All heavy lifting should be done lazily on the first Decode() call if
 * possible.
 */
FVideoDecoderH264_WindowsImpl::FVideoDecoderH264_WindowsImpl()
{
	bIsInitialized = false;
	DecodeMode = EDecodeMode::Undefined;
	bTerminateWorkerThread = false;
	DecoderOutputStreamInfo = {};
	bIsHardwareAccelerated = false;
	bRequiresReconfigurationForSW = false;
	CurrentApplicationOutputSample = nullptr;
	CurrentSoftwareDecoderOutput = nullptr;
	NumFramesInDecoder = 0;
}


/**
 * Destructor. Ideally this should not need to do anything since it is
 * preceded by a Shutdown().
 */
FVideoDecoderH264_WindowsImpl::~FVideoDecoderH264_WindowsImpl()
{
}


/**
 * Called right after the constructor with initialization parameters.
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FVideoDecoderH264_WindowsImpl::Setup(const FInit& InInit)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	// Remember the decoder allocation interface factory methods.
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CreateDecoderAllocationInterfaceFN = InInit.CreateDecoderAllocationInterface;
	ReleaseDecoderAllocationInterfaceFN = InInit.ReleaseDecoderAllocationInterface;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	// [...]
	CreateWorkerThread();
	return true;
}



void FVideoDecoderH264_WindowsImpl::PerformAsyncShutdown()
{
	StopWorkerThread();
	if (bIsInitialized)
	{
		ReleaseD3DResources();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ReleaseDecoderAllocationInterface();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		StaticReleaseResources();
		bIsInitialized = false;
	}
	delete this;
}


void FVideoDecoderH264_WindowsImpl::Shutdown()
{
	// Perform the shutdown asynchronously to work around the potential issue of there
	// being references held to the decoded IMFSample somewhere in the application, which
	// will prevent the decoder transform from coming out of the ProcessOutput() method.
	// In that case the shutdown would be blocked forever which is why we move it to another thread.
	HANDLE th = CreateThread(NULL, 1<<20, &FVideoDecoderH264_WindowsImpl::AsyncShutdownProc, this, 0, NULL);
	if (th)
	{
		CloseHandle(th);
	}
}


bool FVideoDecoderH264_WindowsImpl::SetupFirstUseResources()
{
	if (!bIsInitialized)
	{
		if (StaticInitializeResources() &&
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			CreateDecoderAllocationInterface() &&
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			CreateD3DResources())
		{
			bIsInitialized = true;
		}
	}
	return bIsInitialized;
}



/**
 * Called from webrtc indirectly to decode a frame.
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVideoDecoder::EDecodeResult FVideoDecoderH264_WindowsImpl::Decode(const FVideoDecoderInput* InInput)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	bool bOk = SetupFirstUseResources();
	if (bOk)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!InInput->GetData() || InInput->GetDataSize() <= 0)
		{
			return FVideoDecoder::EDecodeResult::Failure;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// Need to wait for the decoder thread to have spun up??

		TUniquePtr<FInputToDecode> Input = MakeUnique<FInputToDecode>();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Input->PTS = InInput->GetPTS();
		Input->Width = InInput->GetWidth();
		Input->Height = InInput->GetHeight();
		Input->bIsKeyframe = InInput->IsKeyframe();
		Input->Data.AddUninitialized(InInput->GetDataSize());
		FMemory::Memcpy(Input->Data.GetData(), InInput->GetData(), InInput->GetDataSize());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		InputQueue.Enqueue(MoveTemp(Input));

		WorkerThreadSignalHaveWork.Signal();

		// Get all the finished output and send it.
		FVideoDecoderOutputH264_Windows* Output = nullptr;
		while(OutputQueue.Dequeue(Output))
		{
			if (Output->GetIsValid())
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				OnDecodedFrame(Output);
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			
			else
			{
				delete Output;
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				return FVideoDecoder::EDecodeResult::Failure;
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
			}
			
			Output = nullptr;
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return FVideoDecoder::EDecodeResult::Success;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FVideoDecoder::EDecodeResult::Failure;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}





} // namespace AVEncoder
