// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Decoders/VideoDecoderH264_Impl.h"


namespace AVEncoder
{

class FVideoDecoderH264_Windows : public FVideoDecoderH264_Impl
{
public:
	// register decoder with video decoder factory
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	static void Register(FVideoDecoderFactory& InFactory);
	virtual bool Setup(const FInit& InInit) = 0;
	virtual EDecodeResult Decode(const FVideoDecoderInput* InInput) = 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual void Shutdown() = 0;

protected:
	virtual ~FVideoDecoderH264_Windows() = default;
};

} // namespace AVEncoder
