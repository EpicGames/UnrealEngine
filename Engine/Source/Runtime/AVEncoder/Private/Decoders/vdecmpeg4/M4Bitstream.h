// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"
#include "vdecmpeg4_Stream.h"
#include "M4Memory.h"

namespace vdecmpeg4
{

#if 1	// Little endian platform
inline uint32 XSWAP(uint32 value)
{
	return (value << 24) | ((value & 0xff00) << 8) | ((value >> 8) & 0xff00) | (value >> 24);
}
#else
#define XSWAP(a)	(a)
#endif


class M4Bitstream
{
public:
	//! Construction of bitstream object
	M4Bitstream();

	//! Destructor
	~M4Bitstream();

	//! Set memory allocator
	void setMemoryHook(M4MemHandler& memSys);

	//! Link bitstream to provided byte sequence
	void init(const uint8* bitstream, uint32 length);

	//! Link bitstream to stream interface
	void init(VIDStreamIO* pStream, uint32 streamBufferBytes);

	//! Show indicated bit from stream without moving file position
	uint32 show(const uint32 bits);

	//! Skip indicated bits in stream
	void skip(const uint32 bits);

	//! Align bitstream to next byte boundary
	void align();

	//! implements next_start_code()
	void nextStartCode();
	void nextResyncMarker();
	void skipResyncMarker();

	//! implements nextbits_bytealigned()
	uint32 showBitsByteAligned(const uint32 bits);

	bool validStuffingBits();

	//! Return indicated # of bits from stream
	uint32 getBits(const uint32 bits);

	//! Return 1 bit from stream
	uint32 getBit();

	//! Check eof
	bool isEof();

	//! Clear the byte counter
	void totalBitsClear();

	//! Read the byte counter
	uint32 totalBitsGet() const;

	//! Return memory base addres of stream
	const uint32* getBaseAddr() const;

private:
	//! Read from stream in buffered way
	VIDError ReadStreamBuffered(uint32& value);

	//! Copy-constructor is private to prevent usage
	M4Bitstream(const M4Bitstream& pObj);

	//! Assignment operator is private to prevent usage
	const M4Bitstream& operator=(const M4Bitstream& pObj);

	uint32			mAWord;
	uint32			mBWord;

	uint32			mPos;
	uint32*			mTail;

	uint32			mLength;			//!< length of bitstream in bytes
	uint32*			mStart;				//!< pointer to start of data

	VIDStreamIO*	mpStreamIO;			//!< pointer to stream interface for 'pulling' data

	uint8*			mpInternalBuffer;
	uint32			mInternalBufferBytes;
	uint32			mInternalBufferCurrentBytes;
	uint32			mInternalBufferIndex;

	M4MemHandler*	mpMemSys;			//!< memory system used for allocations

	uint32			mTotalBits;			//!< total consumed bits (reset via M4Bitstream::totalBitsClear())

};

/******************************************************************************
 *  INLINE FUNCTIONS
 ******************************************************************************/

// ----------------------------------------------------------------------------
/**
 * Clear byte counter for statistics
 *
 * @return		none
**/
inline void M4Bitstream::totalBitsClear()
{
	mTotalBits = 0;
}




// ----------------------------------------------------------------------------
/**
 * Ctor
**/
inline M4Bitstream::M4Bitstream()
	: mTail(nullptr)
	, mLength(0)
	, mStart(nullptr)
	, mpStreamIO(nullptr)
	, mpInternalBuffer(nullptr)
	, mInternalBufferBytes(0)
	, mInternalBufferCurrentBytes(0)
	, mInternalBufferIndex(0)
	, mpMemSys(nullptr)
	, mTotalBits(0)
{
}


// ----------------------------------------------------------------------------
/**
 * Destructor
**/
inline M4Bitstream::~M4Bitstream()
{
	if (mpInternalBuffer && mpMemSys)
	{
		mpMemSys->free(mpInternalBuffer);
	}
}


// ----------------------------------------------------------------------------
/**
 * Set memory hook for bitstream
 *
 * @param		memSys
 *
 * @return		none
**/
inline void M4Bitstream::setMemoryHook(M4MemHandler& memSys)
{
	mpMemSys = &memSys;
}


/*!
 ******************************************************************************
 * Link bitstream to indicated byte sequence
 *
 * @param bitstream ptr to input bytes
 * @param length number of bytes in input stream
 ******************************************************************************
 */
inline void M4Bitstream::init(const uint8* bitstream, uint32 length)
{
	mStart = mTail = (uint32*)bitstream;
	// initialize 'working' words with 1st 64 bits from stream
	mAWord = XSWAP(mStart[0]);
	mBWord = XSWAP(mStart[1]);
	mPos = 0;

	mLength = length;

	mpStreamIO = nullptr;
	if (mpInternalBuffer)
	{
		mpMemSys->free(mpInternalBuffer);
	}
	mpInternalBuffer = nullptr;
	totalBitsClear();
}


// ----------------------------------------------------------------------------
/**
 * Read uint32 via stream iterator
 *
 * @return		VIDError
**/
inline VIDError M4Bitstream::ReadStreamBuffered(uint32& value)
{
	if (mInternalBufferIndex >= mInternalBufferCurrentBytes)
	{
		mInternalBufferIndex = 0;
		VIDStreamResult result = mpStreamIO->Read(mpInternalBuffer, mInternalBufferBytes, mInternalBufferCurrentBytes);
		if (result != VID_STREAM_OK)
		{
			value = 0;
			mInternalBufferCurrentBytes = 0;
			return result == VID_STREAM_EOF ? VID_ERROR_STREAM_EOF : VID_ERROR_STREAM_ERROR;
		}
		else
		{
			// Check returned data is multiple of 4bytes
			M4CHECK((mInternalBufferCurrentBytes & 0x3) == 0 );
			// Check if returned ANY result
			M4CHECK( mInternalBufferCurrentBytes > 0 );
		}
	}
	value = *((uint32*)(mpInternalBuffer + mInternalBufferIndex));
	mInternalBufferIndex += 4;
	return VID_OK;
}


// ----------------------------------------------------------------------------
/**
 * Handle stream via stream iterator
 *
 * @param		pStream				stream to use
 * @param		streamBufferBytes 	size of intermediate buffer
**/
inline void M4Bitstream::init(VIDStreamIO* pStreamIO, uint32 streamBufferBytes)
{
	M4CHECK(streamBufferBytes > 0);

	mpStreamIO = pStreamIO;

	if (mpInternalBuffer)
	{
		mpMemSys->free(mpInternalBuffer);
	}
	mInternalBufferBytes = streamBufferBytes;
	mpInternalBuffer = (uint8*)mpMemSys->malloc(mInternalBufferBytes);
	mInternalBufferCurrentBytes = 0;
	mInternalBufferIndex = 0;

	// Read initial chunk of data
	uint32 value;
	ReadStreamBuffered(value);
	mAWord = XSWAP(value);

	ReadStreamBuffered(value);
	mBWord = XSWAP(value);

	// initialize 'working' words with 1st 64 bits from stream
	mPos = 0;
	mLength = 0;

	mStart = mTail = nullptr;
	totalBitsClear();
}

/*!
 ******************************************************************************
 * Read # bits from stream, but DON'T move stream pointer
 *
 * @param bits number of bits to read
 *
 * @return Bits to use 'together'
 ******************************************************************************
 */
inline uint32 M4Bitstream::show(const uint32 bits)
{
	M4CHECK(bits>0 && bits<=32);
    uint32 res2;
	int32 nbit = (int32)bits + int32(mPos - 32);
	if (nbit > 0)
	{
		res2 = ((mAWord & (0xffffffff >> mPos)) << nbit) | (mBWord >> (32 - nbit));
	}
	else
	{
		res2 = (mAWord & (0xffffffff >> mPos)) >> (32 - mPos - bits);
	}
	return res2;
}

/*!
 ******************************************************************************
 * Skip indicated number of bits in stream
 *
 * @param bits number of bits to skip
 ******************************************************************************
 */
inline void M4Bitstream::skip(uint32 bits)
{
	M4CHECK(bits>0 && bits<=32);
	mPos += bits;			// advance bit position pointer
	mTotalBits += bits;
	if (mPos >= 32)
	{
		mAWord = mBWord;
		if (mpStreamIO)
		{
			uint32 value;
			ReadStreamBuffered(value);
			mBWord = XSWAP(value);
		}
		else
		{
			mBWord = XSWAP(*(mTail+2));
			mTail++;
		}
		mPos -= 32;
	}
}


// ----------------------------------------------------------------------------
/**
 * Check end of file operation
 *
 * @return		none
**/
inline bool	M4Bitstream::isEof()
{
	return mpStreamIO ? mpStreamIO->IsEof() : ((8 * 4 * (mTail - mStart) + mPos)>>3) >= mLength;
}

/*!
 ******************************************************************************
 * Align bitstream pointer to next byte in stream
 ******************************************************************************
 */
inline void M4Bitstream::align()
{
	uint32 remainder = mPos & 7;
	if (remainder)
	{
		skip(8 - remainder);
	}
}


// ----------------------------------------------------------------------------
/**
 * implements next_start_code()
 */
inline void M4Bitstream::nextStartCode()
{
	uint32 zeroBit = getBit();
	(void)zeroBit;
	M4CHECK(zeroBit == 0);
	while((mPos & 7) != 0)
	{
		uint32 oneBit = getBit();
		(void)oneBit;
		M4CHECK(oneBit == 1);
	}
}

inline void M4Bitstream::nextResyncMarker()
{
	// This is the same as nextStartCode(), actually...
	nextStartCode();
}

inline void M4Bitstream::skipResyncMarker()
{
	// Resync marker is at least 16 zero bits followed by a one.
	// The number of zero bits varies depending on the VOP type, but syntactically it is always the same
	// so we can simply skip over zero bits and one 1 bit.
	while(getBit() == 0)
	{
	}
}


// ----------------------------------------------------------------------------
/**
 * implements nextbits_bytealigned()
 *
 * @param bits
 *
 * @return
 */
inline uint32 M4Bitstream::showBitsByteAligned(const uint32 bits)
{
	M4CHECK(bits>0 && bits<=23);

	uint32 rem = mPos & 7;
	if (rem == 0)
	{
		// Are the next 8 bits an end marker?
		if (show(8) == 0x7f)
		{
			// Yes. This must be skipped and the next n bits returned.
			uint32 v = show(8 + bits);
			return v & (0xffffffff >> (32 - bits));
		}
		else
		{
			return show(bits);
		}
	}
	else
	{
		uint32 v = show(8 - (mPos & 7) + bits);
		return v & (0xffffffff >> (32 - bits));
	}
}

// ----------------------------------------------------------------------------
/**
 * implements valid_stuffing_bits()
 *
 * @return
 */
inline bool M4Bitstream::validStuffingBits()
{
	static const uint8 stuffing[8] = { 0x7f, 0x3f, 0x1f, 0x0f, 0x07, 0x03, 0x01, 0x00 };
	uint32 rb = mPos & 7;
	if (rb == 0)
	{
		return show(8) == 0x7f;
	}
	else
	{
		uint32 rem = show(8 - rb);
		return rem == stuffing[rb];
	}
}


/*!
 ******************************************************************************
 * Return indicated # of bits from stream and move foreward in stream
 *
 * @param bits Number of bits to get
 *
 * @return requested 'bitword'
 ******************************************************************************
 */
inline uint32 M4Bitstream::getBits(const uint32 bits)
{
	uint32 ret = show(bits);
	skip(bits);
	return ret;
}

/*!
 ******************************************************************************
 * Return NEXT bit from stream
 ******************************************************************************
 */
inline uint32 M4Bitstream::getBit()
{
	return getBits(1);
}

// ----------------------------------------------------------------------------
/**
 * Return bits read
 *
 * @return		bit count since last totalBitsClear
**/
inline uint32 M4Bitstream::totalBitsGet() const
{
	return mTotalBits;
}

// ----------------------------------------------------------------------------
/**
 * Return pointer to stream data in memory
**/
inline const uint32* M4Bitstream::getBaseAddr() const
{
	return (const uint32*)mStart;
}


}


