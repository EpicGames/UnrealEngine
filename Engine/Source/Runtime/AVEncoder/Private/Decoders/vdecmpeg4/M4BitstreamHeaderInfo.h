// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4_Types.h"
#include "M4Global.h"

namespace vdecmpeg4
{

struct M4BitstreamHeaderInfo
{
	struct
	{
		uint16 	mQuantType : 1;
		uint32	mRounding  : 1;
	} mFlags;

	uint16		 		mQuant;					// 1-31

	uint8*				mInvQuantIntra;			//!< dequantizer matrix for INTRA block
	uint8*				mInvQuantInter;			//!< dequantizer matrix for INTER block

};

}


