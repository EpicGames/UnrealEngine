// Copyright Epic Games, Inc. All Rights Reserved.

#include "M4BitstreamParser.h"
#include "vdecmpeg4.h"
#include "M4Decoder.h"
#include "M4MemOps.h"

namespace vdecmpeg4
{

#define ROUNDED_DIV(a,b) ( ((a)>0 ? (a) + ((b)>>1) : (a) - ((b)>>1)) / (b) )

#define VIDOBJ_START_CODE		0x00000100		// ..0x0000011f
#define VIDOBJLAY_START_CODE	0x00000120		// ..0x0000012f
#define VISOBJSEQ_START_CODE	0x000001b0
#define VISOBJSEQ_STOP_CODE		0x000001b1
#define VISOBJ_START_CODE		0x000001b5
#define VISOBJ_TYPE_VIDEO				1
#define VIDOBJLAY_TYPE_SIMPLE			1
#define VIDOBJLAY_TYPE_CORE				3
#define VIDOBJLAY_TYPE_MAIN				4
#define VIDOBJLAY_AR_EXTPAR				15
#define VIDOBJLAY_DIVX_UNKNOWN1			17
#define GRPOFVOP_START_CODE		0x000001b3
#define VOP_START_CODE				 0x1b6
#define USERDATA_START_CODE		0x000001b2

#define SKIP_MARKER()	mBitstream->skip(1)


//! Indicated video texture shape
enum
{
	M4_SHAPE_RECT = 0,
	M4_SHAPE_BIN,
	M4_SHAPE_BIN_ONLY,
	M4_SHAPE_GRAY
};

//! Possible sprite types
enum
{
	M4_SPRITE_STATIC = 1,
    M4_SPRITE_GMC
};

const uint32 M4BitstreamParser::mIntraDCThresholdTable[8] =
{
	32,		// value greater than M4MAX allowed quant means 'use intra DC for entire VOP'
	13, 15, 17, 19, 21, 23,
	1		// value lower than M4MIN allowed quant means 'use intra AC for entire VOP'
};

uint8 M4BitstreamParser::mScanTableInput[3][64] =
{
	{	// zig zag
		0,	1,	8,	16, 9,	2,	3,	10,
		17, 24, 32, 25, 18, 11, 4,	5,
		12, 19, 26, 33, 40, 48, 41, 34,
		27, 20, 13, 6,	7,	14, 21, 28,
		35, 42, 49, 56, 57, 50, 43, 36,
		29, 22, 15, 23, 30, 37, 44, 51,
		58, 59, 52, 45, 38, 31, 39, 46,
		53, 60, 61, 54, 47, 55, 62, 63
	},
	{	// horizontal
		0,	1,	2,	3,	8,	9,	16, 17,
		10, 11,	4,	5,	6,	7,	15, 14,
		13, 12, 19, 18, 24, 25, 32, 33,
		26, 27, 20, 21, 22, 23, 28, 29,
		30, 31, 34, 35, 40, 41, 48, 49,
		42, 43, 36, 37, 38, 39, 44, 45,
		46, 47, 50, 51, 56, 57, 58, 59,
		52, 53, 54, 55, 60, 61, 62, 63
	},
	{	// vertical
		0, 8, 16, 24, 1, 9, 2, 10,
		17, 25, 32, 40, 48, 56, 57, 49,
		41, 33, 26, 18, 3, 11, 4, 12,
		19, 27, 34, 42, 50, 58, 35, 43,
		51, 59, 20, 28, 5, 13, 6, 14,
		21, 29, 36, 44, 52, 60, 37, 45,
		53, 61, 22, 30, 7, 15, 23, 31,
		38, 46, 54, 62, 39, 47, 55, 63
	}
};

//! Default MPEG4 dequant coefficients for INTRA blocks
const uint8 M4BitstreamParser::mDefaultIntraMatrix[64] =
{
     8,17,18,19,21,23,25,27,
    17,18,19,21,23,25,27,28,
    20,21,22,23,24,26,28,30,
    21,22,23,24,26,28,30,32,
    22,23,24,26,28,30,32,35,
    23,24,26,28,30,32,35,38,
    25,26,28,30,32,35,38,41,
    27,28,30,32,35,38,41,45
};

//! Default MPEG4 dequant coefficients for INTER blocks
const uint8 M4BitstreamParser::mDefaultInterMatrix[64] =
{
    16,17,18,19,20,21,22,23,
    17,18,19,20,21,22,23,24,
    18,19,20,21,22,23,24,25,
    19,20,21,22,23,24,26,27,
    20,21,22,23,25,26,27,28,
    21,22,23,24,26,27,28,30,
    22,23,24,26,27,28,30,31,
    23,24,25,27,28,30,31,33
};

static const char* M4PictureTypeNames[] =
{
	"I_VOP",
	"P_VOP",
	"B_VOP",
	"S_VOP",
	"N_VOP",
	"SKIP_VOP",
	"ERROR"
};



M4BitstreamParser::M4BitstreamParser()
{
	FMemory::Memzero(&mHeaderInfo, sizeof(mHeaderInfo));
	mBitstream = nullptr;
}

M4BitstreamParser::~M4BitstreamParser()
{
	mDecoder->mMemSys.free(mHeaderInfo.mInvQuantIntra);
	mDecoder->mMemSys.free(mHeaderInfo.mInvQuantInter);
}

// ----------------------------------------------------------------------------
/**
 * Generic parser init
 *
 * @param decoder
 * @param bitstream
 *
 * @return
 */
VIDError M4BitstreamParser::init(M4Decoder* decoder, M4Bitstream* bitstream)
{
	VIDError error = VID_ERROR_OUT_OF_MEMORY;

	mDecoder = decoder;
	mBitstream = bitstream;

	mScanTable[0] = mScanTableInput[0];
	mScanTable[1] = mScanTableInput[1];
	mScanTable[2] = mScanTableInput[2];

	if ((mHeaderInfo.mInvQuantIntra = (uint8*)mDecoder->mMemSys.malloc(sizeof(uint8) * 64)) != nullptr)
	{
		if ((mHeaderInfo.mInvQuantInter = (uint8*)mDecoder->mMemSys.malloc(sizeof(uint8) * 64)) != nullptr)
		{
			mVLCDecoder.init();
			reset();
			return VID_OK;
		}
	}
	mDecoder->mMemSys.free(mHeaderInfo.mInvQuantIntra);
	mDecoder->mMemSys.free(mHeaderInfo.mInvQuantInter);
	FMemory::Memzero(&mHeaderInfo, sizeof(mHeaderInfo));
	return error;
}


// ----------------------------------------------------------------------------
/**
 * Attach parser to new stream
 *
 * @return
 */
VIDError M4BitstreamParser::reset()
{
	mTicksPerSecond = mVopTime = 0.0;
	mLastTimeBase = mTimeBase = 0;
	mLastNonBTime = mTime = 0;
	mTimePP = mTimeBP = 0;

	mVOLfound = false;
	mVOLCounter = 0;

	FMemory::Memzero(&mVOLInfo, sizeof(mVOLInfo));

	mLowDelay = 0;
	mShape = 0;
	mVopTimeIncrementBits = 0;
	mVopTimeIncrementResolution = 0;
	mbVopTimeFixedRate = false;
	mVopTimeFixedIncrement = 0;

	mResyncMacroblockNumber = 0;

	mTicksPerSecond = 0.0;
	mSpriteUsage = 0;
	mQuantBits = 5;

	setMatrix(mHeaderInfo.mInvQuantIntra, mDefaultIntraMatrix);
	setMatrix(mHeaderInfo.mInvQuantInter, mDefaultInterMatrix);

	mQuarterpel = 0;
	mScalability = 0;
	mNewPredEnable = 0;

	mRefVop[0].x = 0;
	mRefVop[0].y = 0;

	mRefVop[1].x = 0;
	mRefVop[1].y = 0;

	mRefVop[2].x = 0;
	mRefVop[2].y = 0;

	mRefVop[3].x = 0;
	mRefVop[3].y = 0;

	return VID_OK;
}


// ----------------------------------------------------------------------------
/**
 * Update parse init if frame size is known
 *
 * @param width
 * @param height
 */
void M4BitstreamParser::initFrame(int16 width, int16 height)
{
	M4CHECK(mVOLInfo.mWidth == width);
	M4CHECK(mVOLInfo.mHeight == height);
	(void)width; (void)height;

	mRefVop[0].x = mRefVop[0].y = 0;

	mRefVop[1].x = mVOLInfo.mWidth;
	mRefVop[1].y = 0;

	mRefVop[2].x = 0;
	mRefVop[2].y = mVOLInfo.mHeight;

	mRefVop[3].x = mVOLInfo.mWidth;
	mRefVop[3].y = mVOLInfo.mHeight;
}


// ----------------------------------------------------------------------------
/**
 * getTrajaPoint
 *
 * @return
 */
int32 M4BitstreamParser::getTrajaPoint()
{
	int32 code = (int32) mBitstream->show(12);
	if (code <= 1023)
	{
		// 00
		mBitstream->skip(2);
		return 0;
	}
	else if (code <= 1535)
	{
		// 010
		mBitstream->skip(3);
		code = (int32) mBitstream->getBit();
		if (code < 1)
		{
			code -= 1;
		}
		return code;
	}
	else if (code <= 2047)
	{
		// 011
		mBitstream->skip(3);
		code = (int32) mBitstream->getBits(2);
		if (code < 2)
		{
			code -= 3;
		}
		return code;
	}
	else if (code <= 2559)
	{
		// 100
		mBitstream->skip(3);
		code = (int32) mBitstream->getBits(3);
		if (code < 4)
		{
			code -= 7;
		}
		return code;
	}
	else if (code <= 3071)
	{
		// 101
		mBitstream->skip(3);
		code = (int32) mBitstream->getBits(4);
		if (code < 8)
		{
			code -= 15;
		}
		return code;
	}
	else if (code <= 3583)
	{
		// 110
		mBitstream->skip(3);
		code = (int32) mBitstream->getBits(5);
		if (code < 16)
		{
			code -= 31;
		}
		return code;
	}
	else if (code <= 3839)
	{
		// 1110
		mBitstream->skip(4);
		code = (int32) mBitstream->getBits(6);
		if (code < 32)
		{
			code -= 63;
		}
		return code;
	}
	else if (code <= 3967)
	{
		// 11110
		mBitstream->skip(5);
		code = (int32) mBitstream->getBits(7);
		if (code < 64)
		{
			code -= 127;
		}
		return code;
	}
	else if (code <= 4031)
	{
		// 111110
		mBitstream->skip(6);
		code = (int32) mBitstream->getBits(8);
		if (code < 128)
		{
			code -= 255;
		}
		return code;
	}
	else if (code <= 4063)
	{
		// 1111110
		mBitstream->skip(7);
		code = (int32) mBitstream->getBits(9);
		if (code < 256)
		{
			code -= 511;
		}
		return code;
	}
	else if (code <= 4079)
	{
		// 11111110
		mBitstream->skip(8);
		code = (int32) mBitstream->getBits(10);
		if (code < 512)
		{
			code -= 1023;
		}
		return code;
	}
	else if (code <= 4087)
	{
		// 111111110
		mBitstream->skip(9);
		code = (int32) mBitstream->getBits(11);
		if (code < 1024)
		{
			code -= 2047;
		}
		return code;
	}
	else if (code <= 4091)
	{
		// 1111111110
		mBitstream->skip(10);
		code = (int32) mBitstream->getBits(12);
		if (code < 2048)
		{
			code -= 4095;
		}
		return code;
	}
	else if (code <= 4093)
	{
		// 11111111110
		mBitstream->skip(11);
		code = (int32) mBitstream->getBits(13);
		if (code < 4096)
		{
			code -= 8191;
		}
		return code;
	}
	else if (code == 4094)
	{
		// 111111111110
		mBitstream->skip(12);
		code = (int32) mBitstream->getBits(14);
		if (code < 8192)
		{
			code -= 16383;
		}
		return code;
	}
	return 0;
}


// ----------------------------------------------------------------------------
/**
 * Parse MPEG4 Elementray Stream
 *
 * @param pictureType
 *
 * @return
 */
VIDError M4BitstreamParser::parseMPEG4ES(M4PictureType& pictureType)
{
	M4CHECK(mBitstream);

	uint32	startCode;
	uint32	vol_ver_id = 1;

	do
	{
		mBitstream->align();
		startCode = mBitstream->show(32);

		/******************************************************************************
		 *	Visual Object Sequence - see MPEG4-P2, 6.2.2
		 *****************************************************************************/
		if (startCode == VISOBJSEQ_START_CODE)
		{
			mBitstream->skip(32);										// visual_object_sequence_start_code
			mVOLInfo.mProfileLevel = (uint8)mBitstream->getBits(8);	// profile_and_level_indication
		}
		else if (startCode == VISOBJSEQ_STOP_CODE)
		{
//check
			mBitstream->skip(32);										// visual_object_sequence_stop_code
		}

		/******************************************************************************
		 *	VisualObject() - see MPEG4-P2, 6.2.2
		 *****************************************************************************/
		else if (startCode == VISOBJ_START_CODE)
		{
			mBitstream->skip(32);										// visual_object_start_code

			if (mBitstream->getBit())									// is_visual_object_identified
			{
				vol_ver_id = mBitstream->getBits(4);					// visual_object_ver_id
				mBitstream->skip(3);									// visual_object_priority
			}
			else
			{
				vol_ver_id = 1;
			}

			if (mBitstream->show(4) != VISOBJ_TYPE_VIDEO)				// visual_object_type
			{
				mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: visual_object_type is not video! (See ISO/IEC 14496-2:2001-6.3.2)\n");
				return VID_ERROR_NOT_VIDEO_STREAM;
			}
			mBitstream->skip(4);

			// video_signal_type()
			if (mBitstream->getBit())									// video_signal_type
			{
				mBitstream->skip(3);					   		 		// video_format (0=component,1=pal,2=ntsc,3=secam,4=MAC,5=unspecified)
				mBitstream->skip(1);									// video_range
				if (mBitstream->getBit())								// color_description
				{
					mBitstream->skip(8);								// color_primaries
					mBitstream->skip(8);								// transfer_characteristics
					mBitstream->skip(8);								// matrix_coefficients
				}
			}
			mBitstream->nextStartCode();
		}
		else if ((startCode & ~0x1fU) == VIDOBJ_START_CODE)
		{
			mBitstream->skip(32);										// video_object_start_code
		}

		/******************************************************************************
		 *	VideoObjectLayer() - see MPEG4-P2, 6.2.3
		 *****************************************************************************/
		else if ((startCode & ~0xfU) == VIDOBJLAY_START_CODE)
		{
			mVOLCounter++;
			mVOLfound = true;

			mBitstream->skip(32);										// video_object_layer_start_code
			mBitstream->skip(1);						   		 		// random_accessible_vol

			// video_object_type_indication
			uint32 voType = mBitstream->getBits(8);

			switch(voType)
			{
				case 0:
				case VIDOBJLAY_TYPE_SIMPLE:
				case VIDOBJLAY_TYPE_CORE:
				case VIDOBJLAY_TYPE_MAIN:
				case VIDOBJLAY_DIVX_UNKNOWN1:
				{
					break;
				}
				default:
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: found unknown video_object_type_indication %d. (See ISO/IEC 14496-2:2001-6.3.3)\n", voType);
					return VID_ERROR_BAD_VIDEO_OBJECT;
				}
			}

			if (mBitstream->getBit())									// is_object_layer_identifier
			{
				vol_ver_id = mBitstream->getBits(4);					// video_object_layer_verid
				mBitstream->skip(3);									// video_object_layer_priority
			}
			else
			{
				vol_ver_id = 1;
			}

			mVOLInfo.mAspectRatio = (uint8)mBitstream->getBits(4);	// aspect_ratio_info

			if (mVOLInfo.mAspectRatio == VIDOBJLAY_AR_EXTPAR)
			{
				mVOLInfo.mAspectRatioPARwidth = (uint8)mBitstream->getBits(8);		// par_width
				mVOLInfo.mAspectRatioPARheight= (uint8)mBitstream->getBits(8);		// par_height
			}

			if (mBitstream->getBit())									// vol_control_parameters
			{
				mBitstream->skip(2);									// chroma_format

				// if low_delay is '1' the VOL contains no B-VOPs!!
				mLowDelay = (uint16)mBitstream->getBit();				// low_delay

				if (mBitstream->getBit())								// vbv_parameters
				{
					mBitstream->skip(15);								// first_half_bitrate
					SKIP_MARKER();
					mBitstream->skip(15);								// latter_half_bitrate
					SKIP_MARKER();
					mBitstream->skip(15);								// first_half_vbv_buffer_size
					SKIP_MARKER();
					mBitstream->skip(3);								// latter_half_vbv_buffer_size
					mBitstream->skip(11);								// first_half_vbv_occupancy
					SKIP_MARKER();
					mBitstream->skip(15);								// latter_half_vbv_occupancy
					SKIP_MARKER();
				}
			}

			mShape = mBitstream->getBits(2);							// video_object_layer_shape

			if (mShape != M4_SHAPE_RECT)
			{
				mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: video_object_layer_shape != \"rectangular\". (See ISO/IEC 14496-2:2001, page 112).\n");
				return VID_ERROR_STREAM_VOL_INVALID_SHAPE;
			}

			if (mShape == M4_SHAPE_GRAY && vol_ver_id != 1)
			{
				mBitstream->skip(4);									// video_object_layer_shape_extension
			}

			SKIP_MARKER();

			// This is a 16-bit unsigned integer that indicates the number of evenly spaced
			// subintervals, called TICKS, within one modulo time. One modulo time represents
			// the fixed interval of one second.
			// In other words, one second is devided into mVopTimeIncRes TICKS
			mVopTimeIncrementResolution = (uint16)mBitstream->getBits(16);			// vop_time_increment_resolution
			mVopTimeIncrementBits = (uint16) log2bin(mVopTimeIncrementResolution - 1);
			if (mVopTimeIncrementBits < 1)
			{
				mVopTimeIncrementBits = 1;
			}

			// Check standard
			mTicksPerSecond = 1.0 / mVopTimeIncrementResolution;

			SKIP_MARKER();

			mbVopTimeFixedRate = mBitstream->getBit() != 0;
			if (mbVopTimeFixedRate)											// fixed_vop_rate
			{
				mVopTimeFixedIncrement = mBitstream->getBits(mVopTimeIncrementBits);	// fixed_vop_time_increment

				mVOLInfo.mFPSNumerator = mVopTimeIncrementResolution;
				mVOLInfo.mFPSDenominator = (uint16) mVopTimeFixedIncrement;
			}

			if (mShape != M4_SHAPE_BIN_ONLY)
			{
				if (mShape == M4_SHAPE_RECT)
				{
					int16 width, height;

					SKIP_MARKER();
					width = (int16) mBitstream->getBits(13);				// video_object_layer_width
					SKIP_MARKER();
					height = (int16) mBitstream->getBits(13);				// video_object_layer_height
					SKIP_MARKER();

					if ((width&0xf) != 0)
					{
						mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: width %d not multiple of 16 pixel!\n", width);
						//return VID_ERROR_WIDTH_OR_HEIGHT_NOT_MULTIPLE_OF_16;
					}

					if ((height&0xf) != 0)
					{
						mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: height %d not multiple of 16 pixel!\n", height);
						//return VID_ERROR_WIDTH_OR_HEIGHT_NOT_MULTIPLE_OF_16;
					}

					mVOLInfo.mCodedWidth = width;
					mVOLInfo.mCodedHeight = height;

					mVOLInfo.mHeight = (height + 15) & ~15;
					mVOLInfo.mWidth = (width + 15) & ~ 15;
				}

				// check if interlace mode and check that interlace mode is'active' for all frames in this movie
				bool interlace = mBitstream->getBit() ? true : false;		// check interlace flag
				if (interlace)
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: interlaced == 1 not supported!\n");
					return VID_ERROR_INTERLACED_NOT_SUPPORTED;
				}


				if (!mBitstream->getBit())										// obmc_disable
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: obmc_disable == 0 ignored. (See ISO/IEC 14496-2:2001, page 113).\n");
				}

				mSpriteUsage = mBitstream->getBits((vol_ver_id == 1 ? 1 : 2));  // sprite_enable

				if (mSpriteUsage == M4_SPRITE_STATIC || mSpriteUsage == M4_SPRITE_GMC)
				{
					if (mSpriteUsage != M4_SPRITE_GMC)
					{
						mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: sprite_enable != \"GMC\". (See ISO/IEC 14496-2:2001, page 113).\n");
						return VID_ERROR_GENERIC;
					}

					if (mSpriteUsage != M4_SPRITE_GMC)
					{
						mBitstream->skip(13);
						SKIP_MARKER();
						mBitstream->skip(13);
						SKIP_MARKER();
						mBitstream->skip(13);
						SKIP_MARKER();
						mBitstream->skip(13);
						SKIP_MARKER();
					}

					mSpriteWarpingPoints = (uint16)mBitstream->getBits(6);
					if (mSpriteWarpingPoints > 3)
					{
						mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: no_of_sprite_warping_points > 3 not valid for GMC sprites! (See ISO/IEC 14496-2:2001, page 114).\n");
						return VID_ERROR_GENERIC;
					}

					mSpriteWarpingAccuracy = (int16)mBitstream->getBits(2);

					uint32 spriteBrightnessChange = mBitstream->getBits(1);
					if (spriteBrightnessChange)
					{
						mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: sprite_brightness_change != 0 not supported! (See ISO/IEC 14496-2:2001, page 114).\n");
						return VID_ERROR_GENERIC;
					}

					if (mSpriteUsage != M4_SPRITE_GMC)
					{
						mBitstream->skip(1);						// low_latency_sprite
					}
				}

				if (vol_ver_id != 1 && mShape != M4_SHAPE_RECT)
				{
					mBitstream->skip(1);							// sadct_disable
				}

				if (mBitstream->getBit())							// not_8_bit
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: IGNORED: not_8_bit == 1!\n");
					mQuantBits = mBitstream->getBits(4);			// quant_precision
					mBitstream->skip(4);							// bits_per_pixel
				}
				else
				{
					mQuantBits = 5;
				}

				if (mShape == M4_SHAPE_GRAY)
				{
					mBitstream->skip(1);							// no_gray_quant_update
					mBitstream->skip(1);							// composition_method
					mBitstream->skip(1);							// linear_composition
				}

				mHeaderInfo.mFlags.mQuantType = (uint16)mBitstream->getBit();					// quant_type

				// if quant_type == 1, we have mpeg4 quant and the possibility of 'different' dequant matrices
				if (mHeaderInfo.mFlags.mQuantType)
				{
					mLoadIntraQuant = mBitstream->getBit()?true:false;	// load_intra_quant_mat
					if (mLoadIntraQuant)
					{
						uint8 matrix[64];
						getMatrix(matrix);
						// install INTRA matrix in quantizer
						setMatrix(mHeaderInfo.mInvQuantIntra, matrix);
					}
					else
					{
						setMatrix(mHeaderInfo.mInvQuantIntra, mDefaultIntraMatrix);
					}

					mLoadInterQuant = mBitstream->getBit()?true:false;	// load_inter_quant_mat
					if (mLoadInterQuant)
					{
						uint8 matrix[64];
						getMatrix(matrix);
						// install INTER matrix in quantizer
						setMatrix(mHeaderInfo.mInvQuantInter, matrix);
					}
					else
					{
						setMatrix(mHeaderInfo.mInvQuantInter, mDefaultInterMatrix);
					}
				}

				if (vol_ver_id != 1)
				{
					// motion compensation either works using 1/2 pel resolution or - if
					// mQuarterpel is set - using 1/4 pel rez.
					mQuarterpel = mBitstream->getBit();			// quarter_sample
					if (mQuarterpel)
					{
						mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: quarter sample mode not supported! (See ISO/IEC 14496-2:2001, page 116).\n");
						return VID_ERROR_GENERIC;
					}
				}
				else
				{
					mQuarterpel = 0;
				}


				// complexity_estimation_disable
				if (!mBitstream->getBit())
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: complexity_estimation_disable == 0 not supported. (See ISO/IEC 14496-2:2001, page 117).\n");
					return VID_ERROR_GENERIC;
				}

				// resync_marker_disable
				if (!mBitstream->getBit())
				{
					//mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: resync_marker_disable == 0 not supported.	(See ISO/IEC 14496-2:2001, page 118).\n");
					//return VID_ERROR_GENERIC;
				}

				// data_partitioned
				uint32 partitioned = mBitstream->getBit();
				if (partitioned)
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: data_partitioned == 1 not supported. (See ISO/IEC 14496-2:2001, page 118).\n");
					return VID_ERROR_GENERIC;
				}

				if (vol_ver_id != 1)
				{
					mNewPredEnable = mBitstream->getBit();
					if (mNewPredEnable)							// newpred_enable
					{
						mBitstream->skip(2);					// requested_upstream_message_type
						mBitstream->skip(1);					// newpred_segment_type
						mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: newpred_enable == 1 not supported. (See ISO/IEC 14496-2:2001, page 118).\n");
						return VID_ERROR_GENERIC;
					}

					// reduced_resolution_vop_enable
					if (mBitstream->getBit())
					{
						mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: reduced_resolution_vop_enable == 1 not supported! (See ISO/IEC 14496-2:2001, page 118).\n");
						return VID_ERROR_GENERIC;
					}
				}

				mScalability = mBitstream->getBit();				// scalability
				if (mScalability)
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: scalability == 1 not supported! (See ISO/IEC 14496-2:2001, page 118).\n");
					return VID_ERROR_GENERIC;
				}
			}
			else	// mShape == M4_SHAPE_BIN_ONLY
			{
				mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: video_object_layer_shape == \"binary only\" not supported!\n");
				return VID_ERROR_STREAM_VOL_INVALID_SHAPE;
			}

			mBitstream->nextStartCode();

			// flag as VOL: Decoder needs to check if video stuff needs to be reinitialized
			pictureType = M4PIC_VOL;

			return VID_OK;
		}

		/******************************************************************************
		 *	Group of Video Object Plane - see MPEG4-P2, 6.2.4
		 *****************************************************************************/
		else if (startCode == GRPOFVOP_START_CODE)
		{
			// Group_of_VideoObjectPlane()
			mBitstream->skip(32);

			// time_code: 18bits
			uint32 hours = mBitstream->getBits(5);
			uint32 minutes = mBitstream->getBits(6);
			SKIP_MARKER();
			uint32 seconds = mBitstream->getBits(6);

			mTimeBase = seconds + 60*(minutes + 60*hours);

			mBitstream->skip(1);								// closed_gov
			mBitstream->skip(1);								// broken_link

			mBitstream->nextStartCode();
		}

		/******************************************************************************
		 *	Video Object Plane- see MPEG4-P2, 6.2.5
		 *****************************************************************************/
		else if (startCode == VOP_START_CODE)
		{
			mBitstream->totalBitsClear();	// reset statistics

			mBitstream->skip(32);										// vop_start_code
			pictureType = (M4PictureType)mBitstream->getBits(2);

			// we must skip all VOPs until we have found a VOL...
			if (mVOLCounter == 0)
			{
				mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: found %s without preceeding VOL header.\n", M4PictureTypeNames[pictureType]);
				return VID_ERROR_STREAM_VOP_WITHOUT_VOL;
			}

			// before we receive the 'Video Object Plane' we must parse a VideoObjectLayer
			if (pictureType == M4PIC_B_VOP && mLowDelay)
			{
				mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: low_delay == 1 and vop_coding_type == \"B\" is invalid! (See ISO/IEC 14496-2:2001, page 111)\n");
				return VID_ERROR_STREAM_ERROR;
			}

			uint32 moduloTimeBase = 0;
			while(mBitstream->getBit())
			{
				++moduloTimeBase;
			}

			SKIP_MARKER();

			uint32 timeIncrement  = mVopTimeIncrementBits > 0 ? mBitstream->getBits(mVopTimeIncrementBits) : 0;	// vop_time_increment
			if (pictureType != M4PIC_B_VOP)
			{
				mLastTimeBase = mTimeBase;
				mTimeBase += moduloTimeBase;
				mTime = mTimeBase * mVopTimeIncrementResolution + timeIncrement;

				if (mTime < mLastNonBTime)
				{
					// !!!BROKEN ENCODER HACK!!!
					mTimeBase++;
					mTime += mVopTimeIncrementResolution;
				}

				mTimePP = mTime - mLastNonBTime;
				mLastNonBTime = mTime;
			}
			else
			{
				mTime = (mLastTimeBase + moduloTimeBase) * mVopTimeIncrementResolution + timeIncrement;
				mTimeBP = mTimePP - (mLastNonBTime - mTime);
				if (mTimePP <= mTimeBP || mTimePP <= mTimePP - mTimeBP || mTimePP <=0)
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: frame skipped (seeking?)\n");
					return VID_ERROR_STREAM_VOP_NOT_CODED;
				}
			}

			mVopTime = mTime * mTicksPerSecond;

			SKIP_MARKER();

			if (!mBitstream->getBit())		// vop_coded
			{
				mBitstream->nextStartCode();
				// ISO/IEC 14496-2:2001:
				// The luminance and chrominance planes of the reconstructed VOP shall be filled with the forward
				// reference VOP as defined in subclause 7.6.7.
				return VID_ERROR_STREAM_VOP_NOT_CODED;
			}

			// newpred_enable
			//  we have already errored out if this is set, so we do not need to handle it here.

			// rounding_type
			mHeaderInfo.mFlags.mRounding = mShape != M4_SHAPE_BIN_ONLY
						&& ((pictureType == M4PIC_P_VOP)
						 || (pictureType == M4PIC_S_VOP && mSpriteUsage == M4_SPRITE_GMC)) ? mBitstream->getBit() : 0;

			// reduced_resolution_vop_enable
			//  we have already errored out if this is set, so we do not need to handle it here.

			if (mShape != M4_SHAPE_RECT)
			{
				if (mSpriteUsage != M4_SPRITE_STATIC || pictureType != M4PIC_I_VOP)
				{
					mBitstream->skip(13);
					SKIP_MARKER();
					mBitstream->skip(13);
					SKIP_MARKER();
					mBitstream->skip(13);
					SKIP_MARKER();
					mBitstream->skip(13);
					SKIP_MARKER();
				}

				// We have already errored out if scalability is set, so we do not need to handle it here.
				/*
					if (mShape != M4_SHAPE_BIN_ONLY && mScalability && enhancement_type)
					{
						mBitstream->skip(1);							// background_composition
					}
				*/

				mBitstream->skip(1);								// change_conv_ratio_disable
				if (mBitstream->getBit())							// vop_constant_alpha
				{
					mBitstream->skip(8);							// vop_constant_alpha_value
				}
			}

			if (mShape != M4_SHAPE_BIN_ONLY)
			{
				// We already errored out if complexity_estimation_disable is set, so we do not need to implement read_vop_complexity_estimation_header()
				//  ...

				// Get a 3-bit code which allows to switch between two VLC’s for coding of Intra DC coefficients
				mIntraDCThreshold = mIntraDCThresholdTable[ mBitstream->getBits(3) ]; // intra_dc_vlc_thr

				// Since interlaced is not supported we do not need to read/skip over
				// 'top_field_first' and 'alternate_vertical_scan_flag'
			}

			if (pictureType == M4PIC_S_VOP && (mSpriteUsage == M4_SPRITE_STATIC || mSpriteUsage == M4_SPRITE_GMC))
			{
				if (mSpriteUsage == M4_SPRITE_STATIC)
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: static sprite in SVOP not supported\n");
					return VID_ERROR_STREAM_ERROR;
				}
				FMemory::Memzero(mSprite, sizeof(mSprite));

				for(uint32 i = 0; i < mSpriteWarpingPoints; i++)
				{
					int32 x, y;

					x = getTrajaPoint();
					SKIP_MARKER();
					y = getTrajaPoint();
					SKIP_MARKER();

					M4CHECK(x > -16383 && x < 16383);
					M4CHECK(y > -16383 && y < 16383);

					mSprite[i].x = (int16)x;
					mSprite[i].y = (int16)y;
				}

				decodeGMCSprite();
			}

			if (mShape != M4_SHAPE_BIN_ONLY)
			{
				mHeaderInfo.mQuant = (uint16)mBitstream->getBits(mQuantBits);					// vop_quant
				if (mHeaderInfo.mQuant < 1 || mHeaderInfo.mQuant > 31)
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: vop_quant value invalid! (See ISO/IEC 14496-2:2001, page 123).\n");
					return VID_ERROR_GENERIC;
				}

				if (mShape == M4_SHAPE_GRAY)
				{
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: gray shape not supported.\n");
					return VID_ERROR_GENERIC;
					/* Otherwise:
						for(i=0; i<aux_comp_count; i++)
							vop_alpha_quant[i]
					*/
				}

				if (pictureType != M4PIC_I_VOP)
				{
					mFcodeForward = (uint16)mBitstream->getBits(3);				// fcode_forward
					if (mFcodeForward < 1 || mFcodeForward > 7)
					{
						mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: vop_fcode_forward value invalid! (See ISO/IEC 14496-2:2001, page 123).\n");
						return VID_ERROR_GENERIC;
					}
				}

				if (pictureType == M4PIC_B_VOP)
				{
					mFcodeBackward = (uint16)mBitstream->getBits(3); 				// fcode_backward
					if (mFcodeBackward < 1 || mFcodeBackward > 7)
					{
						mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: vop_fcode_backward value invalid! (See ISO/IEC 14496-2:2001, page 123).\n");
						return VID_ERROR_GENERIC;
					}
				}

				if (!mScalability)
				{
					if ((mShape != M4_SHAPE_RECT) && (pictureType != M4PIC_I_VOP))
					{
						mBitstream->skip(1);									// vop_shape_coding_type
					}
				}
				else
				{
					// this case cannot happen!!
					mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: scalability == 1 not supported! (See ISO/IEC 14496-2:2001, page 118).\n");
					return VID_ERROR_STREAM_VOP_INVALID_SCALABILITY;
				}

			}
			else	// mShape == M4_SHAPE_BIN_ONLY
			{
				mDecoder->VIDPrintf("M4BitstreamParser::parseMPEG4ES: video_object_layer_shape == \"binary only\" not supported!\n");
				return VID_ERROR_STREAM_VOL_INVALID_SHAPE;
			}

			return VID_OK;
		}

		/******************************************************************************
		 *	User data - see MPEG4-P2, 6.2.2.1
		 *****************************************************************************/
		else if (startCode == USERDATA_START_CODE)
		{
			mBitstream->skip(32);									// user_data_start_code
			while(!mBitstream->isEof() && mBitstream->show(24) != 0x000001)
			{
				mBitstream->skip(8);
			}
		}
		else  // start_code == ?
		{
//check
			if (mBitstream->show(24) == 0x000001)
			{
				// Need to check if this causes too much console spam if the stream is corrupted somehow.
				mDecoder->VIDPrintf("*** M4BitstreamParser::parseMPEG4ES: unknown start_code: 0x%x\n", mBitstream->show(32));
			}

			// While seeking for something valid we do byte-by-byte steps...
			mBitstream->skip(8);
		}
	}
	while(!mBitstream->isEof());

	return VID_ERROR_STREAM_EOF;
}



// ----------------------------------------------------------------------------
/**
 * Parses a video_packet_header()
 * see ISO/IEC 14496-2:2001
 *
 * @return
 */
VIDError M4BitstreamParser::videoPacketHeader()
{
	mBitstream->nextResyncMarker();
	mBitstream->skipResyncMarker();
	if (mShape != M4_SHAPE_RECT)
	{
		mDecoder->VIDPrintf("M4BitstreamParser::videoPacketHeader(): video_object_layer_shape != \"rectangular\" not supported!\n");
		return VID_ERROR_STREAM_VOL_INVALID_SHAPE;
	}
	// macroblock_number. We have already rejected 'reduced_resolution_vop_enable' so we don't need to handle this here.
	mResyncMacroblockNumber = mBitstream->getBits(log2bin(mVOLInfo.mWidth * mVOLInfo.mHeight / 256 - 1));

	if (mShape != M4_SHAPE_BIN_ONLY)
	{
		mHeaderInfo.mQuant = (uint16)mBitstream->getBits(mQuantBits);					// quant_scale (updates vop_quant)
	}
	bool header_extension_code = false;
	if (mShape == M4_SHAPE_RECT)
	{
		header_extension_code = !!mBitstream->getBit();
	}
	if (header_extension_code)
	{
		mDecoder->VIDPrintf("M4BitstreamParser::videoPacketHeader(): header_extension_code == 1 not supported!\n");
		return VID_ERROR_STREAM_ERROR;
		/*
			do {
				modulo_time_base 1 bslbf
			} while(modulo_time_base != ‘0’)
			marker_bit 1 bslbf
			vop_time_increment 1-16 bslbf
			marker_bit 1 bslbf
			vop_coding_type 2 uimsbf
			if (video_object_layer_shape != “rectangular”) {
				change_conv_ratio_disable 1 bslbf
				if (vop_coding_type != “I”)
					vop_shape_coding_type 1 bslbf
			}
			if (video_object_layer_shape != “binary only”) {
				intra_dc_vlc_thr 3 uimsbf
				if (sprite_enable == “GMC” && vop_coding_type == “S” && no_of_sprite_warping_points > 0)
					sprite_trajectory()
				if ((reduced_resolution_vop_enable) && (video_object_layer_shape == “rectangular”) && ((vop_coding_type == “P”) || (vop_coding_type == “I”)))
					vop_reduced_resolution 1 bslbf
				if (vop_coding_type != “I”)
					vop_fcode_forward 3 uimsbf
				if (vop_coding_type == “B”)
					vop_fcode_backward 3 uimsbf
			}
		*/
	}
	if (mNewPredEnable)							// newpred_enable
	{
		// Already checked for earlier. Recheck just in case that changed.
		mDecoder->VIDPrintf("M4BitstreamParser::videoPacketHeader(): newpred_enable == 1 not supported.\n");
		return VID_ERROR_STREAM_ERROR;
		/*
			vop_id 4-15 uimsbf
			vop_id_for_prediction_indication 1 bslbf
			if (vop_id_for_prediction_indication)
				vop_id_for_prediction 4-15 uimsbf
			marker_bit 1 bslbf
		*/
	}
	return VID_OK;
}


// ----------------------------------------------------------------------------
/**
 * Read GMC sprite paramters from file
 * See MPEG4-P2, 6.2.5.4 Sprite coding
 */
void M4BitstreamParser::readGMCSprite()
{
	M4CHECK(mSpriteWarpingPoints < 4);		// GMC sprite only supports max 3 warping points!

	// IMPORTANT, because we use d[0] to d[3], which may not be initialized
	FMemory::Memzero(mSprite, sizeof(mSprite));
	M4_VECTOR* d = mSprite;
	for(uint32 i = 0; i < mSpriteWarpingPoints; i++)
	{
		d[i].x = (int32) mBitstream->getBits(14);
		if (mBitstream->getBit())
		{
			d[i].x = -d[i].x;
		}
		d[i].y = (int32) mBitstream->getBits(14);
		if (mBitstream->getBit())
		{
			d[i].y = -d[i].y;
		}
	}
}


// ----------------------------------------------------------------------------
/**
 * Decode GMC sprite paramters from file
 * See MPEG4-P2, 6.2.5.4 Sprite coding
 */
void M4BitstreamParser::decodeGMCSprite()
{
	M4_VECTOR 	refSprite[4];
	M4_VECTOR	refVirtual[4];
	M4_VECTOR* d = mSprite;

	// we need this because of 'signed' operations below
	int32 width = mVOLInfo.mWidth;
	int32 height = mVOLInfo.mHeight;

	// hm, proper init call missing or what?
	M4CHECK(width > 0 && height > 0);

	int32 rho= 3 - mSpriteWarpingAccuracy;
	int32 alpha = 0, beta = 0;
	while((1 << alpha) < width)
	{
		alpha++;
	}
	while((1 << beta) < height)
	{
		beta++;
	}
	int32 w2 = 1<<alpha;
	int32 h2 = 1<<beta;

	// calculate the sprite reference points
	int32 accuracy = 2 << mSpriteWarpingAccuracy;
	int32 a1 = accuracy >> 1;
	refSprite[0].x = a1 * (2 * mRefVop[0].x + d[0].x);
	refSprite[0].y = a1 * (2 * mRefVop[0].y + d[0].y);

	refSprite[1].x = a1 * (2 * mRefVop[1].x + d[0].x + d[1].x);
	refSprite[1].y = a1 * (2 * mRefVop[1].y + d[0].y + d[1].y);

	refSprite[2].x = a1 * (2 * mRefVop[2].x + d[0].x + d[2].x);
	refSprite[2].y = a1 * (2 * mRefVop[2].y + d[0].y + d[2].y);

	// the idea behind this virtual_ref mess is to be able to use shifts later per pixel instead of divides
	// so the distance between points is converted from w&h based to w2&h2 based which are of the 2^x form
	int32 r = 16 / accuracy;

	int32 tmp = (width - w2) * (r*refSprite[0].x - 16*mRefVop[0].x) + w2 * (r*refSprite[1].x - 16*mRefVop[1].x);
	refVirtual[0].x = 16*(mRefVop[0].x + w2) + ROUNDED_DIV(tmp, width);

	tmp = (width - w2) * (r*refSprite[0].y - 16*mRefVop[0].y) + w2 * (r*refSprite[1].y - 16*mRefVop[1].y);
	refVirtual[0].y = 16*mRefVop[0].y + ROUNDED_DIV(tmp ,width);

	tmp = (height - h2) * (r*refSprite[0].x - 16*mRefVop[0].x) + h2*(r*refSprite[2].x - 16*mRefVop[2].x);
	refVirtual[1].x = 16*mRefVop[0].x + ROUNDED_DIV(tmp, height);

	tmp = (height - h2) * (r*refSprite[0].y - 16*mRefVop[0].y) + h2 * (r*refSprite[2].y - 16*mRefVop[2].y);
	refVirtual[1].y = 16*(mRefVop[0].y + h2) + ROUNDED_DIV(tmp ,height);

	FMemory::Memzero(mSpriteDelta, sizeof(mSpriteDelta));
	FMemory::Memzero(&mSpriteShift, sizeof(mSpriteShift));

	switch (mSpriteWarpingPoints)
	{
		case 0:
		{
			mSpriteOffset[0].x = 0;
			mSpriteOffset[0].y = 0;
			mSpriteOffset[1].x = 0;
			mSpriteOffset[1].y = 0;

			mSpriteDelta[0].x = accuracy;
			mSpriteDelta[0].y = 0;

			mSpriteDelta[1].x = 0;
			mSpriteDelta[1].y = accuracy;

			mSpriteShift.x = 0;
			mSpriteShift.y = 0;
			break;
		}

		case 1:	//GMC only
		{
			mSpriteOffset[0].x = refSprite[0].x - accuracy * mRefVop[0].x;
			mSpriteOffset[0].y = refSprite[0].y - accuracy * mRefVop[0].y;

			mSpriteOffset[1].x = ((refSprite[0].x>>1) | (refSprite[0].x&1)) - accuracy*( mRefVop[0].x / 2 );
			mSpriteOffset[1].y = ((refSprite[0].y>>1) | (refSprite[0].y&1)) - accuracy*( mRefVop[0].y / 2 );

			mSpriteDelta[0].x = accuracy;
			mSpriteDelta[0].y = 0;
			mSpriteDelta[1].x = 0;
			mSpriteDelta[1].y = accuracy;
			mSpriteShift.x = 0;
			mSpriteShift.y = 0;
			break;
		}

		case 2:
		{
			mSpriteOffset[0].x = (refSprite[0].x<<(alpha+rho))
									+ (-r*refSprite[0].x + refVirtual[0].x)*(-mRefVop[0].x)
									+ ( r*refSprite[0].y - refVirtual[0].y)*(-mRefVop[0].y)
									+ (1<<(alpha+rho-1));
			mSpriteOffset[0].y = (refSprite[0].y<<(alpha+rho))
									+ (-r*refSprite[0].y + refVirtual[0].y)*(-mRefVop[0].x)
									+ (-r*refSprite[0].x + refVirtual[0].x)*(-mRefVop[0].y)
									+ (1<<(alpha+rho-1));
			mSpriteOffset[1].x = ( (-r*refSprite[0].x + refVirtual[0].x)*(-2*mRefVop[0].x + 1)
									  +( r*refSprite[0].y - refVirtual[0].y)*(-2*mRefVop[0].y + 1)
									  +2*w2*r*refSprite[0].x - 16*w2
									  + (1<<(alpha+rho+1)));
			mSpriteOffset[1].y = ( (-r*refSprite[0].y + refVirtual[0].y)*(-2*mRefVop[0].x + 1)
									  +(-r*refSprite[0].x + refVirtual[0].x)*(-2*mRefVop[0].y + 1)
									  +2*w2*r*refSprite[0].y - 16*w2
									  + (1<<(alpha+rho+1)));

			mSpriteDelta[0].x = (-r*refSprite[0].x + refVirtual[0].x);
			mSpriteDelta[0].y = ( r*refSprite[0].y - refVirtual[0].y);
			mSpriteDelta[1].x = (-r*refSprite[0].y + refVirtual[0].y);
			mSpriteDelta[1].y = (-r*refSprite[0].x + refVirtual[0].x);

			mSpriteShift.x = alpha+rho;
			mSpriteShift.y = alpha+rho+2;
			break;
		}

		case 3:
		{
			int32 min_ab = M4MIN(alpha, beta);
			int32 w3 = w2>>min_ab;
			int32 h3 = h2>>min_ab;
			mSpriteOffset[0].x =  (refSprite[0].x<<(alpha+beta+rho-min_ab))
									 + (-r*refSprite[0].x + refVirtual[0].x)*h3*(-mRefVop[0].x)
									 + (-r*refSprite[0].x + refVirtual[1].x)*w3*(-mRefVop[0].y)
									 + (1<<(alpha+beta+rho-min_ab-1));
			mSpriteOffset[0].y =  (refSprite[0].y<<(alpha+beta+rho-min_ab))
									 + (-r*refSprite[0].y + refVirtual[0].y)*h3*(-mRefVop[0].x)
									 + (-r*refSprite[0].y + refVirtual[1].y)*w3*(-mRefVop[0].y)
									 + (1<<(alpha+beta+rho-min_ab-1));
			mSpriteOffset[1].x =  (-r*refSprite[0].x + refVirtual[0].x)*h3*(-2*mRefVop[0].x + 1)
									 + (-r*refSprite[0].x + refVirtual[1].x)*w3*(-2*mRefVop[0].y + 1)
									 + 2*w2*h3*r*refSprite[0].x
									 - 16*w2*h3
									 + (1<<(alpha+beta+rho-min_ab+1));
			mSpriteOffset[1].y =  (-r*refSprite[0].y + refVirtual[0].y)*h3*(-2*mRefVop[0].x + 1)
									 + (-r*refSprite[0].y + refVirtual[1].y)*w3*(-2*mRefVop[0].y + 1)
									 + 2*w2*h3*r*refSprite[0].y
									 - 16*w2*h3
									 + (1<<(alpha+beta+rho-min_ab+1));
			mSpriteDelta[0].x = (-r*refSprite[0].x + refVirtual[0].x)*h3;
			mSpriteDelta[0].y = (-r*refSprite[0].x + refVirtual[1].x)*w3;
			mSpriteDelta[1].x = (-r*refSprite[0].y + refVirtual[0].y)*h3;
			mSpriteDelta[1].y = (-r*refSprite[0].y + refVirtual[1].y)*w3;

			mSpriteShift.x = alpha + beta + rho - min_ab;
			mSpriteShift.y = alpha + beta + rho - min_ab + 2;
			break;
		}
	}

	if (mSpriteDelta[0].x == accuracy<<mSpriteShift.x && mSpriteDelta[0].y == 0 &&
	    mSpriteDelta[1].x == 0          && mSpriteDelta[1].y == accuracy<<mSpriteShift.x)
	{
		mSpriteOffset[0].x >>= mSpriteShift.x;
		mSpriteOffset[0].y >>= mSpriteShift.x;
		mSpriteOffset[1].x >>= mSpriteShift.y;
		mSpriteOffset[1].y >>= mSpriteShift.y;
		mSpriteDelta[0].x= accuracy;
		mSpriteDelta[0].y= 0;
		mSpriteDelta[1].x= 0;
		mSpriteDelta[1].y= accuracy;
		mSpriteShift.x = 0;
		mSpriteShift.y = 0;

		mSpriteWarpingPointsUsed = 1;
	}
	else
	{
		int32 shift_y = 16 - mSpriteShift.x;
		int32 shift_c = 16 - mSpriteShift.y;

		mSpriteOffset[0].x <<= shift_y;
		mSpriteOffset[0].y <<= shift_y;

		mSpriteOffset[1].x <<= shift_c;
		mSpriteOffset[1].y <<= shift_c;

		mSpriteDelta[0].x <<= shift_y;
		mSpriteDelta[0].y <<= shift_y;

		mSpriteDelta[1].x <<= shift_y;
		mSpriteDelta[1].y <<= shift_y;

		mSpriteShift.x = 16;
		mSpriteShift.y = 16;

		mSpriteWarpingPointsUsed = mSpriteWarpingPoints;
	}
}

// ------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------
// ------------------------------------------------------------------------------------------------

M4BitstreamCache::M4BitstreamCache()
	: mpCacheEntry(nullptr)
{
}

M4BitstreamCache::~M4BitstreamCache()
{
}

VIDError M4BitstreamCache::Init(M4MemHandler& memSys)
{
	if ((mpCacheEntry = (M4BitstreamCacheEntry*)memSys.malloc(sizeof(M4BitstreamCacheEntry), 128)) != nullptr)
	{
		return VID_OK;
	}
	return VID_ERROR_OUT_OF_MEMORY;
}

void M4BitstreamCache::Exit(M4MemHandler& memSys)
{
	memSys.free(mpCacheEntry);
}

}

