// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"
#include "M4Global.h"
#include "M4Bitstream.h"
#include "M4VlcDecoder.h"
#include "M4BitstreamHeaderInfo.h"

namespace vdecmpeg4
{

#define M4_CLIP(X,A)		(X > A) ? (A) : (X)
#define M4_ABS(X) 			(((X)>0)?(X):-(X))


class M4Decoder;

//! Picture type in input stream to decode
enum M4PictureType
{
	M4PIC_I_VOP = 0,
	M4PIC_P_VOP,
	M4PIC_B_VOP,
	M4PIC_S_VOP,
	M4PIC_VOL,
};


/*
 ******************************************************************************
 * This class handles the entire parsing process of the input stream.
 * Currently, we're reading an MPEG4 compatible bitstream.
 *
 * Note: This class is not designed for inheritance.
 ******************************************************************************
 */
class M4BitstreamParser
{
public:
	//! Default constructor
	M4BitstreamParser();

	//!	Destructor
	~M4BitstreamParser();

	//! Perform initialization of data structures
	VIDError init(M4Decoder* decoder, M4Bitstream* bitstream);

	//! Attach parser to new stream
	VIDError reset();

	//! Handle change/update of frame parameters
	void initFrame(int16 width, int16 height);

	//! Parsing of full MPEG4-ES
	VIDError parseMPEG4ES(M4PictureType& pictureType);

	//! Scan stream for next valid startcode
	VIDError findNextStartCode(uint32& absolutePos);

	int32 getCbpCIntra()
	{
		uint32 index;
		while((index = mBitstream->show(9)) == 1)
		{
			mBitstream->skip(9);
		}
		index >>= 3;
		mBitstream->skip(VLC_LEN(mVLCDecoder.mTabCbpCIntra[index]));
		return VLC_CODE(mVLCDecoder.mTabCbpCIntra[index]);
	}

	int32 getCbpCInter()
	{
		uint32 index;
		while((index = M4_CLIP(mBitstream->show(9), 256)) == 1)
		{
			mBitstream->skip(9);
		}
		mBitstream->skip(VLC_LEN(mVLCDecoder.mTabCbpCInter[index]));
		return VLC_CODE(mVLCDecoder.mTabCbpCInter[index]);
	}

	uint8 getMBType()
	{
		uint8 type;
		for(type = 0; type <= 3; type++)
		{
			if (mBitstream->getBit())
			{
				break;
			}
		}
		M4CHECK(type <= 3);
		return type;
	}

	/*
	 ******************************************************************************
	 * get CBPY from stream
	 *
	 * 	This variable length code represents a pattern of non-transparent
	 * 	blocks with at least one non intra DC transform
	 * 	coefficient, in a macroblock.
	 ******************************************************************************
	 */
	uint32 getCbpy(bool intra)
	{
		uint32 index = mBitstream->show(6);
		mBitstream->skip(VLC_LEN(mVLCDecoder.mTabCbpY[index]));
		uint32 cbpy = VLC_CODE(mVLCDecoder.mTabCbpY[index]);
		return intra ? cbpy : 15 - cbpy;
	}


	uint16 getDCScaler(uint8 quant, bool luminance)
	{
		if (quant > 0 && quant < 5)
		{
			return 8;
		}
		if (quant < 25 && !luminance)
		{
			return (quant + 13) >> 1;
		}
		if (quant < 9)
		{
			return (uint16)(quant << 1);
		}
		if (quant < 25)
		{
			return quant + 8;
		}
		return luminance ? (uint16)(quant << 1) - 16 : quant - 6;
	}

	int32 getDCSize(bool luminance)
	{
		uint32 code, i;

		if (luminance)
		{
			code = mBitstream->show(11);
			for (i = 11; i > 3; i--)
			{
				if (code == 1)
				{
					mBitstream->skip(i);
					return (int32)(i + 1);
				}
				code >>= 1;
			}
			mBitstream->skip(VLC_LEN(mVLCDecoder.mDCLumTab[code]));
			return VLC_CODE(mVLCDecoder.mDCLumTab[code]);
		}
		else
		{
			code = mBitstream->show(12);
			for(i = 12; i > 2; i--)
			{
				if (code == 1)
				{
					mBitstream->skip(i);
					return (int32)i;
				}
				code >>= 1;
			}
			return 3 - (int32)mBitstream->getBits(2);
		}
	}

	int32 getDCDiff(uint32 dc_size)
	{
		M4CHECK(dc_size < 32U);
		int32 code = (int32)mBitstream->getBits(dc_size);
		int32 msb = code >> (dc_size - 1);
		return msb == 0 ? -1 * (code^((1<<dc_size) - 1)) : code;
	}

	void decodeInterBlock(int16* block)
	{
		getInterBlockNoAsm(block);
	}

	void decodeIntraBlock(int16* block, uint32 direction, uint32 startCoeff)
	{
		getIntraBlockNoAsm(block, direction, startCoeff);
	}

	int32 getQuantiserChange()
	{
		if ( !mBitstream->getBit() )			// '0'
		{
			return 0;
		}
		else if (!mBitstream->getBit() )		// '10'
		{
			return -2;
		}
		else
		{
			return 2;							// '11'
		}
	}

	/*
	 ******************************************************************************
	 * Get a coded INTRA block from bitstream
	 *
	 * Note: We assume that the input block is set to 0.
	 *
	 * @param block
	 *		ptr to int16 to receive coefficients from from bitstream
	 * @param direction
	 *		0: zigZag scan
	 *		1: horizontal scan
	 *		2: vertical scan
	 * @param startCoeff
	 *		where to start in block
	 ******************************************************************************
	**/
	void getIntraBlockNoAsm(int16* block, uint32 direction, uint32 startCoeff)
	{
		const uint8* scan = mScanTable[direction];
		int32 last, run;

		do
		{
			int32 level = mVLCDecoder.getCoeffIntraNoAsm(run, last, *mBitstream);
			M4CHECK((level >= -2047) && (level <= 2047) && "getIntraBlock: intra_overflow!!");
			M4CHECK(run != -1 && "getIntraBlock: invalid run");
			M4CHECK(run >= 0 && "getIntraBlock: invalid run");	// JPCHANGE
			startCoeff += (uint32)run;						// number of '0's to skip
			M4CHECK(startCoeff<64);
			block[ scan[startCoeff] ] = (int16)level;		// set to current level
			startCoeff++;
		}
		while(!last);		// last == 0: "there are more nonzero coefficients in this block"
	}

	void getInterBlockNoAsm(int16* block)
	{
		const uint8* scan = mScanTable[0];
		int32 run, last;
		int32 p = 0;

		do
		{
			int32 level = mVLCDecoder.getCoeffInterNoAsm(run, last, *mBitstream);
			M4CHECK((level >= -2047) && (level <= 2047) && "getInterBlock: level overflow!!");
			M4CHECK(run != -1 && "getInterBlock: invalid run");
			p += run;
			M4CHECK(p<64);
			block[ scan[p] ] = (int16)level;
			p++;
		}
		while(!last);	// last == 0: "there are more nonzero coefficients in this block"
	}

	int32 getMv(const uint16 code)
	{
		int32 res;
		int32 mv;
		int32 scale_fac = 1 << (code - 1);

		int32 codeMagnitude = getMvVlc();

		if (scale_fac == 1 || codeMagnitude == 0)
		{
			return codeMagnitude;
		}

		// read motion vector residual
		M4CHECK(code <= 32U);
		res = (int32)mBitstream->getBits(code - 1);
		mv = ((M4_ABS(codeMagnitude) - 1) * scale_fac) + res + 1;

		return codeMagnitude < 0 ? -mv : mv;
	}

	int16 GetWidth() const
	{
		return mVOLInfo.mWidth;
	}

	int16 GetHeight() const
	{
		return mVOLInfo.mHeight;
	}

	double GetLastVopTime() const
	{
		return mVopTime;
	}

	const VIDStreamEvents::VOLInfo& GetVOLInfo() const
	{
		return mVOLInfo;
	}

	VIDError videoPacketHeader();

private:
	//! Read the data from the vidConv stream
	void readGMCSprite();

	//! Decode GMC sprite
	void decodeGMCSprite();

	//! Install dequant matrix
	void setMatrix(uint8* matrixOut, const uint8* matrixIn)
	{
		for(uint32 i=0; i<64; ++i)
		{
			matrixOut[i] = matrixIn[i];
		}
	}

	//! Convert from log to binary
	uint32 log2bin(uint32 value)
	{
		uint32 n = 0;
		while(value)
		{
			value >>= 1;
			++n;
		}
		return n;
	}

	//! Read matrix from stream
    void getMatrix(uint8* matrix)
	{
		uint32 last;
		uint32 i = 0;
		uint32 value = 0;
		do
		{
			last = value;
			value = mBitstream->getBits(8);
			matrix[mScanTable[0][i++]] = (uint8)value;
		}
		while(value != 0 && i < 64);

		// 'fill-up' end of matrix using last
		i--;
		while(i < 64)
		{
			matrix[mScanTable[0][i++] ] = (uint8)last;
		}
	}

	int32 getTrajaPoint();
	int32 getMvVlc()
	{
		uint32 index;

		if (mBitstream->getBit())
		{
			return 0;	// Vector difference == 0
		}

		index = mBitstream->show(12);

		if (index >= 512)
		{
			index = (index >> 8) - 2;
			mBitstream->skip(VLC_LEN(mVLCDecoder.mTabTMNMV0[index]));
			return (int16)(VLC_CODE(mVLCDecoder.mTabTMNMV0[index]));
		}

		if (index >= 128)
		{
			index = (index >> 2) - 32;
			mBitstream->skip(VLC_LEN(mVLCDecoder.mTabTMNMV1[index]));
			return (int16)VLC_CODE(mVLCDecoder.mTabTMNMV1[index]);
		}

		index -= 4;
		mBitstream->skip(VLC_LEN(mVLCDecoder.mTabTMNMV2[index]));
		return (int16)VLC_CODE(mVLCDecoder.mTabTMNMV2[index]);
	}

	//! Copy-constructor not implemented
	M4BitstreamParser(const M4BitstreamParser& pObj);

	//! Assignment operator not implemented
	const M4BitstreamParser& operator=(const M4BitstreamParser& pObj);

public:
	M4BitstreamHeaderInfo	 mHeaderInfo;
	VIDStreamEvents::VOLInfo mVOLInfo;

	uint16		 		mLowDelay;					//!< set to '1' indicates the VOL contains **NO** B-VOPs!
	uint16		 		mFcodeForward;
	uint16				mFcodeBackward;

	uint32				mIntraDCThreshold;
	uint32				mScalability;

	uint16				mVopTimeIncrementBits;
	uint16				mVopTimeIncrementResolution;
	uint32				mVopTimeFixedIncrement;
	bool				mbVopTimeFixedRate;

	double				mTicksPerSecond;
	double				mVopTime;

	uint32				mLastTimeBase;
	uint32				mTimeBase;

	uint64				mTime;
	uint64				mLastNonBTime;

	uint64				mTimePP;
	uint64				mTimeBP;

	uint32				mResyncMacroblockNumber;

	uint32				mSpriteUsage;

	uint16				mSpriteWarpingPoints;
	uint16				mSpriteWarpingPointsUsed;

	int16				mSpriteWarpingAccuracy;
	int16				_free;

	M4_VECTOR			mSpriteOffset[2];
	M4_VECTOR 			mSpriteDelta[2];
	M4_VECTOR 			mSpriteShift;

private:
	M4Bitstream*		mBitstream;				//!< ptr to bitstream object

	uint32	   		 	mShape;
	uint32	 		   	mQuantBits;
	uint32	  		  	mQuarterpel;
	uint32				mNewPredEnable;

	uint8*				mScanTable[3];			//!< scan table relocated into locked cache

	bool				mVOLfound;
	uint64				mVOLCounter;

	bool				mLoadIntraQuant;
	bool				mLoadInterQuant;

	uint8				mIntraMatrix[64];
	uint8				mInterMatrix[64];

	M4_VECTOR 			mSprite[4];

	M4VlcDecoder		mVLCDecoder;

	M4_VECTOR			mRefVop[4];

	static uint8 		mScanTableInput[3][64];

	static const uint32	mIntraDCThresholdTable[8];
	static const uint8	mDefaultIntraMatrix[64];
	static const uint8	mDefaultInterMatrix[64];

	M4Decoder*				mDecoder;
};









class M4BitstreamCache
{
public:
	M4BitstreamCache();
	~M4BitstreamCache();

	VIDError Init(M4MemHandler& memSys);
	void Exit(M4MemHandler& memSys);

	M4BitstreamCacheEntry& Alloc()
	{
		FMemory::Memzero(mpCacheEntry->mDctFromBitstream, sizeof(mpCacheEntry->mDctFromBitstream));
		return *mpCacheEntry;
	}

private:
	M4BitstreamCacheEntry*	mpCacheEntry;
};


}


