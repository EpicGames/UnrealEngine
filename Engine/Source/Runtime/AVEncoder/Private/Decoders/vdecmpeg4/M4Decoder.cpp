// Copyright Epic Games, Inc. All Rights Reserved.
#include "M4Decoder.h"

#include "vdecmpeg4.h"
#include "M4Bitstream.h"
#include "M4Image.h"
#include "M4Prediction.h"
#include "M4MotionVectorMgr.h"
#include "M4idct.h"
#include "M4InvQuant.h"
#include "M4MemOps.h"


namespace vdecmpeg4
{

#define DEFAULT_STREAM_BUFFER_BYTES 2048

#define M4RSHIFT(a,b) ((a) > 0 ? ((a) + (1<<((b)-1)))>>(b) : ((a) + (1<<((b)-1))-1)>>(b))
#define M4SIGN(X) (((X)>0)?1:-1)
#define M4ABS(X)	(((X)>0)?(X):-(X))

const int32 	M4Decoder::mDequantTable[4] = { -1, -2, 1, 2 };
const uint32 	M4Decoder::mRoundtab[16] = { 0, 0, 0, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 1, 2, 2 };
const uint8 	M4Decoder::mIntraMacroBlocks[8] = { 0, 0, 0, 1, 1, 0, 0, 0 };


// ----------------------------------------------------------------------------
/**
 * Static creation
 *
 * @param memHandler
 *
 * @return
 */
M4Decoder* M4Decoder::create(M4MemHandler& memHandler)
{
    return new(memHandler) M4Decoder(memHandler);
}


// ----------------------------------------------------------------------------
/**
 * Static destruction
 *
 * @param pDecoder
 * @param memHandler
 */
void M4Decoder::destroy(M4Decoder*& pDecoder, M4MemHandler& memHandler)
{
	if (pDecoder)
	{
		pDecoder->~M4Decoder();
		memHandler.free(pDecoder);
		pDecoder = nullptr;
	}
}


void M4Decoder::VIDPrintf(const char* pFormat, ...)
{
	if (mCbPrintf)
	{
		char buffer[512];
		va_list args;
		va_start(args, pFormat);
		vsnprintf(buffer, 512, pFormat, args);
		va_end(args);
		buffer[511] = '\0';
		mCbPrintf(buffer);
	}
}


// ----------------------------------------------------------------------------
/**
 * Constructor
 *
 * @param memHandler
 */
M4Decoder::M4Decoder(M4MemHandler& memHandler)
	: mDecoderFlags(0)
	, mIPMacroblocks(0)
	, mBMacroblocks(0)
	, mMotVecMgr(0)
	, mpStreamIO(nullptr)
	, mpStreamEvents(nullptr)
	, mMemSys(memHandler)
{
	static_assert(sizeof(M4_MB) == 256, "Size mismatch");
	mCurrent = mTempImage[0] = mTempImage[1] = mReference[0] = mReference[1] = 0;

	mpVidImage = nullptr;
	mNumVidImages = 0;

	// Important: Now need todo allocs inside bitstream
	mBitstream.setMemoryHook(mMemSys);

#ifdef _M4_ENABLE_BMP_OUT
	mBaseName[0] = '\0';
#endif
}


// ----------------------------------------------------------------------------
/**
 * Destructor
 */
M4Decoder::~M4Decoder()
{
	mXCommand.Exit();
	freeStuff();
}


// ----------------------------------------------------------------------------
/**
 * Just release image buffer stuff
 */
void M4Decoder::freeBuffers()
{
	for(uint32 i=0; i<2; ++i)
	{
		if (mReference[i])
		{
			mReference[i]->RefRemove();
			mReference[i] = nullptr;
		}
	}

	mCurrent = nullptr;

	if (mpVidImage)
	{
		for(uint32 i=0; i<mNumVidImages; ++i)
		{
			mpVidImage[i]->RefRemove();
		}
		mMemSys.free(mpVidImage);
		mpVidImage = nullptr;
	}

	M4Image::destroy(mTempImage[0], this);
	M4Image::destroy(mTempImage[1], this);

	mMemSys.free(mIPMacroblocks);
	mIPMacroblocks = nullptr;
	mMemSys.free(mBMacroblocks);
	mBMacroblocks = nullptr;
}


// ----------------------------------------------------------------------------
/**
 * Release all buffers
 */
void M4Decoder::freeStuff()
{
	freeBuffers();
	mBitstreamCache.Exit(mMemSys);
    M4MotionVectorMgr::destroy(mMotVecMgr, mMemSys);
}


// ----------------------------------------------------------------------------
/**
 * Handle init code executed just once
 *
 * @param setup  width/height in setup struct can be 0
 *
 * @return none
 */
VIDError M4Decoder::init(const VIDDecoderSetup* setup)
{
	M4CHECK(setup);

	VIDError error = VID_OK;

	mFrameCounter = 0;
	mWidth 		  = 0;
	mHeight 	  = 0;
	mDecoderFlags = setup->flags;

	// Create at least 3 buffers here
	mNumVidImages = mDecoderFlags & VID_DECODER_VID_BUFFERS ? setup->numOfVidBuffers : 3;
	if (mNumVidImages < 3)
	{
		return VID_ERROR_SETUP_NUMBER_OF_VID_BUFFERS_INVALID;
	}

	// Initialize the 'protocol' parser stuff
	error = mBitstreamParser.init(this, &mBitstream);
	if (error == VID_OK)
	{
		error = mBitstreamCache.Init(mMemSys);
		if (error == VID_OK)
		{
			error = VID_ERROR_OUT_OF_MEMORY;
			mMotVecMgr = M4MotionVectorMgr::create(this, mMemSys);
			if (mMotVecMgr)
			{
				error = mXCommand.Init(this);
				if (error == VID_OK)
				{
					return VID_OK;
				}
			}
		}
	}

	// Error handling...
	freeStuff();
	return error;
}


// ----------------------------------------------------------------------------
/**
 * Handle buffer allocation of frame size changes
 *
 * @param width  new width
 * @param height new height
 *
 * @return VIDError code
 */
VIDError M4Decoder::initBuffers(int16 width, int16 height)
{
	// Check minimum size
	M4CHECK(width>=32 && height>=32);
	// Check for multiple of 16.
	M4CHECK((width &0xf)==0);
	M4CHECK((height&0xf)==0);

	// Check if we really have a change in the dimension.
	if (mpVidImage != nullptr && width == mWidth && height == mHeight)
	{
		// Try to check if stream was changed, but buffers are not reallocated
		if (mBitstreamParser.GetLastVopTime() > 0.0)
		{
			// We assume that this is an 'inbetween' VOL and actually do not need to change state (is this correct?)
			return VID_OK;
		}
		else
		{
			// Frame buffer size is valid, but need to reset what else?
			return VID_OK;
		}
	}

	mWidth 	= width;
	mHeight = height;

	// Release any old stuff
	freeBuffers();

	// Create at least 2 (non-B frames) or 3 (B-frames) buffers here
	mpVidImage = (M4Image**)mMemSys.malloc(sizeof(M4Image*) * mNumVidImages);
	for(uint32 i=0; i < mNumVidImages; ++i)
	{
		mpVidImage[i] = M4Image::create(this, mWidth, mHeight);
		if (!mpVidImage[i])
		{
			freeBuffers();
			return VID_ERROR_OUT_OF_MEMORY;
		}
	}

	// Images are created with a ref count of 1
	mCurrent = mpVidImage[0];

	mReference[0] = mpVidImage[1];
	mReference[0]->RefAdd(); 		// fake: image is referenced by decoder

	mReference[1] = mpVidImage[2];
	mReference[1]->RefAdd();	// fake: image is referenced by decoder

	mBitstreamParser.initFrame(mWidth, mHeight);

	// convert input size to 16x16 macroblock size
	mMBWidth  = (uint16)((mWidth +15)>>4);
	mMBHeight = (uint16)((mHeight+15)>>4);

	mIPMacroblocks = (M4_MB*)mMemSys.malloc(mMBWidth * mMBHeight * sizeof(M4_MB));
	if (!mIPMacroblocks)
	{
		freeBuffers();
		return VID_ERROR_OUT_OF_MEMORY;
	}

	mTempImage[0] = M4Image::create(this, mWidth, mHeight);	// interpolate mode B-frame
	mTempImage[1] = M4Image::create(this, mWidth, mHeight);	// additional output buffer
	mBMacroblocks = (M4_MB*)mMemSys.malloc(mMBWidth * mMBHeight * sizeof(M4_MB));
	if (!mTempImage[0] || !mTempImage[1] || !mBMacroblocks)
	{
		freeBuffers();
		return VID_ERROR_OUT_OF_MEMORY;
	}

	mCurrent = nullptr;
	return VID_OK;
}


// ----------------------------------------------------------------------------
/**
 * Allocate a buffer for a new frame
 *
 * @return
 */
M4Image* M4Decoder::AllocVidFrame()
{
	for(uint32 i=0; i<mNumVidImages; ++i)
	{
		M4CHECKF(mpVidImage[i]->RefGet() >= 1, TEXT("Refcount error! Someone decremented twice %d"), mpVidImage[i]->RefGet());
		M4CHECKF(mpVidImage[i]->RefGet() <= 3, TEXT("Refcount error! Someone incremented too often (todo!) %d"), mpVidImage[i]->RefGet());
	}

	for(uint32 i=0; i<mNumVidImages; ++i)
	{
		// Free images are having a ref count of 1 for meaning: "just allocated and pointed to by mpVidImage"
		if (mpVidImage[i]->RefGet() == 1)
		{
			return mpVidImage[i];
		}
	}
	return nullptr;
}


// ----------------------------------------------------------------------------
/**
 * Attach stream to decoder.
 *
 * Ownership is not transferred
 *
 * @param pStreamIO
 * @param pEvents
 *
 * @return VIDError code
 */
VIDError M4Decoder::StreamSet(VIDStreamIO* pStreamIO, VIDStreamEvents* pEvents)
{
	mpStreamIO = pStreamIO;
	// Attach stream to stream object
	if (mpStreamIO)
	{
		mBitstream.init(mpStreamIO, DEFAULT_STREAM_BUFFER_BYTES);
		mBitstreamParser.reset();
	}
	mpStreamEvents = pEvents;
	return VID_OK;
}


// ----------------------------------------------------------------------------
/**
 * Handle decoding via stream interface
 *
 * @param result
 *
 * @return VIDError code
 */
VIDError M4Decoder::StreamDecode(float /*time*/, const VIDImage** result)
{
	if (mpStreamIO == nullptr && mBitstream.getBaseAddr() == nullptr)
	{
		return VID_ERROR_STREAM_NOT_SET;
	}

	*result = nullptr;

	VIDImageInfo* pImageInfo;
	VIDError error = VID_OK;
	M4Image* pReturnedImage;	// frame returned to the user

	M4PictureType type;

	// Try to alloc a buffer where to put the final frame
	// This is only a check meaning that the buffer is NOT marked as 'reserved' here
	M4Image* pFinishedImage = nullptr;
	if (mpVidImage)
	{
		pFinishedImage = AllocVidFrame();
		if (pFinishedImage == nullptr)
		{
			return VID_ERROR_DECODE_NO_VID_BUFFER_AVAILABLE;
		}
	}

	do
	{
		// Parse mpeg4-es skipping any non-coded vop
		do
		{
			error = mBitstreamParser.parseMPEG4ES(type);
		}
		while(error == VID_ERROR_STREAM_VOP_NOT_CODED);

		// Check if there was any other error decoding the header
		if (error != VID_OK)
		{
			return error;
		}

		// VOL processing
		if (type == M4PIC_VOL)
		{
			// Handle information callback to user
			if (mpStreamEvents)
			{
				mpStreamEvents->FoundVideoObjectLayer(mBitstreamParser.GetVOLInfo());
			}

			error = initBuffers(mBitstreamParser.GetWidth(), mBitstreamParser.GetHeight());
			if (error != VID_OK)
			{
				return error;
			}

			pFinishedImage = AllocVidFrame();
			if (pFinishedImage == nullptr)
			{
				return VID_ERROR_DECODE_NO_VID_BUFFER_AVAILABLE;
			}
		}

	}
	while(type == M4PIC_VOL);	// scan again if type as a VOL...

	// VOP processing
	if (type < M4PIC_I_VOP || type > M4PIC_S_VOP)
	{
		return VID_ERROR_DECODE_INVALID_VOP;
	}

	// Set location where we put the output image
	M4CHECK(pFinishedImage);
	mCurrent = pFinishedImage;
	pFinishedImage = nullptr;

	// Check if we have allocated additional buffers
	mXCommand.FrameBegin(mCurrent, &mBitstreamParser.mHeaderInfo, mReference);

	if (type == M4PIC_P_VOP)
	{
		error = pFrame(false);
	}
	else if (type == M4PIC_I_VOP)
	{
		error = iFrame();
	}
	else if (type == M4PIC_B_VOP)
	{
		error = bFrame();
	}
	else
	{
		error = pFrame(true);	// S(GMC)-VOP
	}
	if (error != VID_OK)
	{
		return error;
	}

	pImageInfo = &mCurrent->mImageInfo;
	pImageInfo->mFrameNumber = mFrameCounter;

	mCurrent->mImage.time = mBitstreamParser.GetLastVopTime();

	// swap images for next decoder call
	// if we have a b frame, we don't SWAP buffers. We reuse mCurrent for the next frame
	if (type != M4PIC_B_VOP)
	{
		// In B-Frame supporting mode we need to 'delay' one frame, because we
		// get two P-frames for constructing a possible following B-frame.
		*result = &mReference[0]->mImage;

		// This image is handed out to user
		mReference[0]->RefAdd();

		// This image is used sometimes later as reference frame
		mCurrent->RefAdd();

		// This image is not used anymore by the decoder (but perhaps by the user?)
		mReference[1]->RefRemove();

		mReference[1] = mReference[0];
		mReference[0] = mCurrent;
	}
	else
	{
		// Special case for "single consecutive b-frame" mode
		*result = &mCurrent->mImage;
		// Image is handed out to user
		mCurrent->RefAdd();
	}
	mCurrent = nullptr;
	mFrameCounter++;

	mXCommand.FrameEnd();

	// Update stats
	pImageInfo->mFrameBytes = mBitstream.totalBitsGet()>>3;
	pImageInfo->mMacroblockInfo = mXCommand.GetMacroblockInfo();

	// Finally, check if this actuall was a valid
	pReturnedImage = (M4Image*)(*result)->_private;
	M4CHECK(pReturnedImage);
	if (pReturnedImage->mImageInfo.mFrameType == VID_IMAGEINFO_FRAMETYPE_UNKNOWN)
	{
		pReturnedImage->RefRemove();
		return VID_ERROR_STREAM_UNDERFLOW;
	}
	else
	{
		// Debugging: Check if we want to write a bmp image from the created image
#ifdef _M4_ENABLE_BMP_OUT
		if (mBaseName[0] && pReturnedImage)
		{
			// FrameCount is already increment here. So, we just use previous 'count'
			pReturnedImage->saveBMP(mBaseName, mFrameCounter-1);
		}
#endif
		return VID_OK;
	}
}


// ----------------------------------------------------------------------------
/**
 * Do some stuff if seeking happens.
 *
 * WARNING: Need to revise interface if seeking to some arbitraty position
 *
 * @return
 */
VIDError M4Decoder::StreamSeekNotify()
{
	if (mpStreamIO)
	{
		mBitstream.init(mpStreamIO, DEFAULT_STREAM_BUFFER_BYTES);
	}
	mBitstreamParser.reset();

	// Need to invalidate all frames if initialized...
	if (mpVidImage)
	{
		// Mark all frames as invalid, because we do a seek now.
		for(uint32 i=0; i < mNumVidImages; ++i)
		{
			mpVidImage[i]->mImageInfo.mFrameType = VID_IMAGEINFO_FRAMETYPE_UNKNOWN;
			mpVidImage[i]->Black();
		}
		freeBuffers();
	}
	return VID_OK;
}


// ----------------------------------------------------------------------------
/**
 * Decode I-Frame
 *
 * @return
 */
VIDError M4Decoder::iFrame()
{
	mCurrent->mImageInfo.mFrameType = VID_IMAGEINFO_FRAMETYPE_I;

	uint16 quant = mBitstreamParser.mHeaderInfo.mQuant;
	M4_MB* mb = mIPMacroblocks;
	for(int32 y=0; y<mMBHeight; ++y)
	{
		for(int32 x=0; x<mMBWidth; ++x, ++mb)
		{
			// get macroblock TYPE **AND** the coded block pattern for chrominance .
			// It is always included for coded macroblocks.
			mb->mFlags = 0;
			int32 mbcbpc = mBitstreamParser.getCbpCIntra();
			mb->mMode = (uint8)(mbcbpc & 7);
			mb->mModeIntra = mIntraMacroBlocks[mb->mMode];
			M4CHECK(mb->mMode != M4_MBMODE_STUFFING);
			if (mb->mMode == M4_MBMODE_STUFFING)
			{
				VIDPrintf("M4Decoder::iFrame(): Stuffing not supported!\n");
				return VID_ERROR_DECODE_STUFFING_NOT_SUPPORTED;
			}

			uint32 cbpc = (uint32)(mbcbpc >> 4);

			// This is a 1-bit flag which when set to '1' indicates that either the first row or
			// the first column of ac coefficients are differentially coded for intra coded macroblocks.
			bool useACPredition = !!mBitstream.getBit();

			// This variable length code represents a pattern of non-transparent luminance blocks
			// with at least one non intra DC transform coefficient, in a macroblock.
			mb->mCbp = (uint8)((mBitstreamParser.getCbpy(true) << 2) | cbpc);

			if (mb->mMode == M4_MBMODE_INTRA_Q)
			{
				// This is a 2-bit code which specifies the change in the quantizer, for I- and P-VOPs!
				quant += (int16)mDequantTable[mBitstream.getBits(2)];
				if (quant > 31)
				{
					quant = 31;
				}
				else if (quant < 1)
				{
					quant = 1;
				}
			}
			mb->mQuant = (uint8)quant;

			mbIntra(mb, x, y, useACPredition, mb->mCbp, (uint8)quant);

			// Resync marker?
			if (mBitstream.showBitsByteAligned(17) == 1 && mBitstream.validStuffingBits())
			{
				VIDError err = mBitstreamParser.videoPacketHeader();
				if (err != VID_OK)
				{
					return err;
				}
				// The current assumption is that the next macroblock is also the one we want to handle in our loop.
				M4CHECK(mBitstreamParser.mResyncMacroblockNumber == y * mMBWidth + x + 1);
			}
/*
			else if (mBitstream.showBitsByteAligned(23) == 0)
			{
				// Next aligned 23 bits are all 0 which indicates a startcode. The frame is either complete or not.
				// For now we assume it is.
				M4CHECK(y == mMBHeight-1 && x == mMBWidth - 1);
				// If it is not we can still bail out here with some indication of this.
				mBitstream.nextStartCode();
				return VID_OK;
			}
*/
		}
	}
	mBitstream.nextStartCode();
	return VID_OK;
}


// ----------------------------------------------------------------------------
/**
 * Decode P-Frame
 *
 * @param gmc
 *
 * @return
 */
VIDError M4Decoder::pFrame(bool gmc)
{
	mCurrent->mImageInfo.mFrameType = gmc ? VID_IMAGEINFO_FRAMETYPE_S : VID_IMAGEINFO_FRAMETYPE_P;

	// init motion vector manager for this frame
	mMotVecMgr->init();

	// Update/create border padding area of image
	mXCommand.XCreatePadding(&mReference[0]->mImage);

	uint8 quant = (uint8)mBitstreamParser.mHeaderInfo.mQuant;
	M4_MB* mb = mIPMacroblocks;
	for(int32 mby=0; mby<mMBHeight; ++mby)
	{
		for(int32 mbx=0; mbx<mMBWidth; ++mbx, ++mb)
		{
			mb->mFlags = 0;

			// check for flag 'not coded'
			// 'not coded' means that we do not have ANY info about this mb in this frame
			if (mBitstream.getBit())		// not_coded
			{
				// this case: not_coded == 1
				if (!gmc)
				{
					mb->mMode = M4_MBMODE_NOT_CODED_PVOP;
					mb->mModeIntra = 0;
					// copy this macroblock's data directly from previous VOP
					M4MotionVectorClear(mb);
					mXCommand.XCopyMB(mbx, mby);
				}
				else
				{
					return VID_ERROR_DECODE_GMC_NOT_ENABLED;
				}
			}
			else
			{
				int32 mbcbpc = mBitstreamParser.getCbpCInter();
				mb->mMode = (uint8)(mbcbpc & 7);
				mb->mModeIntra = mIntraMacroBlocks[mb->mMode];
				M4CHECKF(mb->mMode != M4_MBMODE_STUFFING, TEXT("*** M4Decoder::pFrame: STUFFING currently not supported!\n"));
				if (mb->mMode == M4_MBMODE_STUFFING)
				{
					VIDPrintf("M4Decoder::pFrame(): Stuffing not supported!\n");
					return VID_ERROR_DECODE_STUFFING_NOT_SUPPORTED;
				}

				uint32 cbpc = (uint32)(mbcbpc >> 4);

				// handle this macroblock as INTRA
				if (mb->mModeIntra)
				{
					bool useACPredition = !!mBitstream.getBit();

					// get 'coded block pattern' for luminance
					uint32 cbpy = mBitstreamParser.getCbpy(true);
					mb->mCbp = (uint8)((cbpy << 2) | cbpc);

					if (mb->mMode == M4_MBMODE_INTRA_Q)
					{
						quant += (int8)mDequantTable[mBitstream.getBits(2)];
						if (quant > 31)
						{
							quant = 31;
						}
						else if (quant < 1)
						{
							quant = 1;
						}
					}
					mb->mQuant = quant;

					mbIntra(mb, mbx, mby, useACPredition, mb->mCbp , quant);
				}
				else
				{
					// handle this macroblock as INTER (incl. INTER-GMC)
					uint32 mcsel = gmc && ( mb->mMode == M4_MBMODE_INTER || mb->mMode == M4_MBMODE_INTER_Q ) ? mBitstream.getBit() : 0;	// mcsel

					uint32 cbpy = mBitstreamParser.getCbpy(false);
					mb->mCbp = (uint8)((cbpy << 2) | cbpc);

					// change quantiser value
					if (mb->mMode == M4_MBMODE_INTER_Q)
					{
						quant += (int8)mDequantTable[mBitstream.getBits(2)];
						if (quant > 31)
						{
							quant = 31;
						}
						else if (quant < 1)
						{
							quant = 1;
						}
					}
					mb->mQuant = quant;

					if (!mcsel || mb->mMode == M4_MBMODE_INTER4V /* new: this overwrites mcsel */)
					{
						mMotVecMgr->readMvForward(mb);
						mMotVecMgr->predictAddForward(mb, mbx, mby);
						mbInter(mb, mbx, mby, mb->mCbp, MV_PREDICTION_P_FORWARD, 0);
					}
					else
					{
						return VID_ERROR_DECODE_GMC_NOT_ENABLED;
					}
				}
			}

			// Resync marker?
			if (mBitstream.showBitsByteAligned(16 + mBitstreamParser.mFcodeForward) == 1 && mBitstream.validStuffingBits())
			{
				VIDError err = mBitstreamParser.videoPacketHeader();
				if (err != VID_OK)
				{
					return err;
				}
				// The current assumption is that the next macroblock is also the one we want to handle in our loop.
				M4CHECK(mBitstreamParser.mResyncMacroblockNumber == mby * mMBWidth + mbx + 1);
			}
/*
			else if (mBitstream.showBitsByteAligned(23) == 0)
			{
				// Next aligned 23 bits are all 0 which indicates a startcode. The frame is either complete or not.
				// For now we assume it is.
				M4CHECK(mby == mMBHeight-1 && mbx == mMBWidth - 1);
				// If it is not we can still bail out here with some indication of this.
				mBitstream.nextStartCode();
				return VID_OK;
			}
*/
		}	// mbx loop
	} 	// mby loop
	mBitstream.nextStartCode();
	return VID_OK;
}


// ----------------------------------------------------------------------------
/**
 * Decode B-Frame
 *
 * @return
 */
VIDError M4Decoder::bFrame()
{
	mCurrent->mImageInfo.mFrameType = VID_IMAGEINFO_FRAMETYPE_B;

	// Init motion vector manager for this frame
	mMotVecMgr->init();

	// Update/create border padding area of image
	// In a multiprocessor environment, this is a NOP, because padding happens
	// in a separate thread as soon as the image is returned to the user...
	mXCommand.XCreatePadding(&mReference[0]->mImage);

	bool hasMotionVec;
	uint16 mbLastIdx = 0;
	M4_MB* mbLast = mIPMacroblocks;
	M4_MB* mb     = mBMacroblocks;
	uint8 quant = (uint8)mBitstreamParser.mHeaderInfo.mQuant;

	int32 TRD = (int32)mBitstreamParser.mTimePP;
	int32 TRB = (int32)mBitstreamParser.mTimeBP;
	M4_VECTOR			mMvForwardPred;
	M4_VECTOR			mMvBackwardPred;

	for(int32 mby=0; mby<mMBHeight; ++mby)
	{
		// Init motion vector predictor
		mMvForwardPred = mMvBackwardPred = M4MotionVectorMgr::MV_ZERO;

		for(int32 mbx=0; mbx<mMBWidth; ++mbx, ++mb, ++mbLast, ++mbLastIdx)
		{
			M4_VECTOR deltaMv = M4MotionVectorMgr::MV_ZERO;
			mb->mMode 	   = M4_MBMODE_INTER;	// default to this mode
			mb->mModeIntra = 0;

			// If the co-located macroblock in the most recently decoded I- or P-VOP is skipped...
			if (mbLast->mMode == M4_MBMODE_NOT_CODED_PVOP)
			{
				// If the co-located macroblock in the most recently decoded I- or P-VOP is skipped,
				// the current B-macroblock is treated as the forward mode with the zero motion vector (MVFx,MVFy)
				mb->mFlags = M4_MBFLAG_MVTYPE_FORWARD;
				mb->mCbp   = 0;
				mb->mQuant = mbLast->mQuant;
				M4MotionVectorClear(mb);
				mbInter(mb, mbx, mby, mb->mCbp, MV_PREDICTION_NONE, 1);
				continue;
			}

			if (mBitstream.getBit())	// modb
			{
				// If modb == 1 the current B macroblock is reconstructed by using the direct mode with zero delta vector.
				hasMotionVec = false;

				mb->mFlags = M4_MBFLAG_MVTYPE_DIRECT;
				mb->mCbp   = 0;
				deltaMv    = M4MotionVectorMgr::MV_ZERO;
			}
			else
			{
				hasMotionVec = true;

				// here mb_type is present, but we must check for cbpb
				uint8 modb1 = (uint8)mBitstream.getBit();

				// Get b macroblock type: 00, 01, 10, 11
				mb->mFlags = mBitstreamParser.getMBType() | M4_MBFLAG_HAS_MOTION_VECTOR;

				mb->mCbp = (uint8)(modb1 ? 0 : mBitstream.getBits(6));

				if ( ((mb->mFlags & M4_MBFLAG_MVTYPE_MASK) != M4_MBFLAG_MVTYPE_DIRECT) && !modb1 )
				{
					quant += (int8)mBitstreamParser.getQuantiserChange();
					if (quant > 31)
					{
						quant = 31;
					}
					else if (quant < 1)
					{
						quant = 1;
					}
				}
            }

			// store current quant value in this macroblock
			mb->mQuant = quant;

			// --------------------------------------------------------------------------------------------------------

			switch(mb->mFlags & M4_MBFLAG_MVTYPE_MASK)
			{
				case M4_MBFLAG_MVTYPE_DIRECT:
				{
                    if (hasMotionVec)
					{
						mMotVecMgr->readMv(deltaMv, 1);
						mMotVecMgr->addMv(deltaMv, 1, deltaMv, M4MotionVectorMgr::MV_ZERO);
					}

					// progressive direct mode
					for(uint32 i = 0; i < 4; i++)
					{
						mb->mFMv[i].x = TRB * mbLast->mFMv[i].x / TRD + deltaMv.x;
						mb->mFMv[i].y = TRB * mbLast->mFMv[i].y / TRD + deltaMv.y;

						mb->mBMv[i].x = deltaMv.x == 0
												? (TRB - TRD) * mbLast->mFMv[i].x / TRD
												: mb->mFMv[i].x - mbLast->mFMv[i].x;

						mb->mBMv[i].y = deltaMv.y == 0
												? (TRB - TRD) * mbLast->mFMv[i].y / TRD
												: mb->mFMv[i].y - mbLast->mFMv[i].y;
					}

					mb->mMode = M4_MBMODE_INTER4V;
					mb->mModeIntra = 0;

					mbInterpolate(1, 0, mb, mbx, mby, mb->mCbp, mbLastIdx);
					break;
				}

				case M4_MBFLAG_MVTYPE_INTERPOLATE:
				{
					mMotVecMgr->readMv(mb->mFMv[0], mBitstreamParser.mFcodeForward);
					mMotVecMgr->addMv(mb->mFMv[0], mBitstreamParser.mFcodeForward, mb->mFMv[0], mMvForwardPred);
					mMvForwardPred = mb->mFMv[1] = mb->mFMv[2] = mb->mFMv[3] = mb->mFMv[0];

					mMotVecMgr->readMv(mb->mBMv[0], mBitstreamParser.mFcodeBackward);
					mMotVecMgr->addMv(mb->mBMv[0], mBitstreamParser.mFcodeBackward, mb->mBMv[0], mMvBackwardPred);
					mMvBackwardPred = mb->mBMv[1] = mb->mBMv[2] = mb->mBMv[3] = mb->mBMv[0];

					mbInterpolate(1, 0, mb, mbx, mby, mb->mCbp);
                    break;
				}

				case M4_MBFLAG_MVTYPE_BACKWARD:
				{
					mMotVecMgr->readMv(mb->mFMv[0], mBitstreamParser.mFcodeBackward);
					mMotVecMgr->addMv(mb->mFMv[0], mBitstreamParser.mFcodeBackward, mb->mFMv[0], mMvBackwardPred);
					mMvBackwardPred = mb->mFMv[1] = mb->mFMv[2] = mb->mFMv[3] = mb->mFMv[0];

					mbInter(mb, mbx, mby, mb->mCbp, MV_PREDICTION_B_BACKWARD, 0);	// calc from 'future' I- or P-VOP
					break;
				}

				case M4_MBFLAG_MVTYPE_FORWARD:
				{
					mMotVecMgr->readMv(mb->mFMv[0], mBitstreamParser.mFcodeForward);

					mMotVecMgr->addMv(mb->mFMv[0], mBitstreamParser.mFcodeForward, mb->mFMv[0], mMvForwardPred);
					mMvForwardPred = mb->mFMv[1] = mb->mFMv[2] = mb->mFMv[3] = mb->mFMv[0];

					mbInter(mb, mbx, mby, mb->mCbp, MV_PREDICTION_B_FORWARD, 1);	// calc from 'past' I- or P-VOP
					break;
				}

				default:
				{
					M4CHECK(false && "Invalid B-frame type\n");
					break;
				}
			}
#if 0
			// Resync marker?
			uint32 resyncMarkerLen = max(15 + mBitstreamParser.mFcodeForward + mBitstreamParser.mFcodeBackward, 17) + 1;
			if (mBitstream.showBitsByteAligned(resyncMarkerLen == 1 && mBitstream.validStuffingBits())
			{
				VIDError err = mBitstreamParser.videoPacketHeader();
				if (err != VID_OK)
				{
					return err;
				}
				// The current assumption is that the next macroblock is also the one we want to handle in our loop.
				M4CHECK(mBitstreamParser.mResyncMacroblockNumber == mby * mMBWidth + mbx + 1);
			}
/*
			else if (mBitstream.showBitsByteAligned(23) == 0)
			{
				// Next aligned 23 bits are all 0 which indicates a startcode. The frame is either complete or not.
				// For now we assume it is.
				M4CHECK(mby == mMBHeight-1 && mbx == mMBWidth - 1);
				// If it is not we can still bail out here with some indication of this.
				mBitstream.nextStartCode();
				return VID_OK;
			}
*/
#endif
		}
	}
	mBitstream.nextStartCode();
	return VID_OK;
}


// ----------------------------------------------------------------------------
/**
 * Decode an INTRA macroblock
 *
 * @param mb
 * @param mbx
 * @param mby
 * @param useACPredition
 * @param cbp
 * @param quant
 */
void M4Decoder::mbIntra(M4_MB* mb, int32 mbx, int32 mby, bool useACPredition, uint32 cbp, uint8 quant)
{
	M4CHECK(mb->mMode == M4_MBMODE_INTRA || mb->mMode == M4_MBMODE_INTRA_Q);
	M4CHECK(mb->mModeIntra);

	// Get a new cache entry where to store our bitstream data
	M4BitstreamCacheEntry& cacheEntry = mBitstreamCache.Alloc();

	int16* pDctBitstream = cacheEntry.mDctFromBitstream;
	uint16* iDcScaler    = cacheEntry.mDcScaler;

	// no motion vector for this macroblock
	M4MotionVectorClear(mb);

	int16	predictionValues[8];

	uint32	dctOffset = 0;
	uint32 cbpMask = 32;
	for(uint32 i=0; i<6; ++i, dctOffset += 64, cbpMask >>=1)
	{

		iDcScaler[i] = mBitstreamParser.getDCScaler(mb->mQuant, i < 4);

		M4PredictionInit(mIPMacroblocks, mbx, mby, mMBWidth, i, mb->mQuant, iDcScaler[i], predictionValues);

		if (!useACPredition)
		{
			mb->mACPredDirections[i] = 0;
		}

		uint32 startCoeff = 0;
		if (quant < mBitstreamParser.mIntraDCThreshold)
		{
			int32 dcSize = mBitstreamParser.getDCSize(i < 4);
			int32 dcDif = dcSize ? mBitstreamParser.getDCDiff((uint32) dcSize) : 0;

			if (dcSize > 8)
			{
				mBitstream.skip(1);				// skip marker bit
			}

			pDctBitstream[dctOffset] = (int16)dcDif;	// DC coeff set (= read from stream)

			startCoeff = 1;						// start at first AC component
		}

		if (cbp & cbpMask)					// coded
		{
			mBitstreamParser.decodeIntraBlock(pDctBitstream+dctOffset, mb->mACPredDirections[i], startCoeff);
		}

		M4PredictionAdd(mb, pDctBitstream+dctOffset, i, iDcScaler[i], predictionValues);
	}
	mXCommand.XUpdateIntraMB(mb, mbx, mby, &cacheEntry);
}




// ----------------------------------------------------------------------------
/**
 * Decode an INTER macroblock
 *
 * @param mb
 * @param mbx
 * @param mby
 * @param cbp
 * @param mvPrediction
 * @param refImgNo
 */
void M4Decoder::mbInter(M4_MB* mb, int32 mbx, int32 mby, uint32 cbp, MV_PREDICTION mvPrediction, uint32 refImgNo)
{
	M4CHECK(refImgNo < 2);
	M4CHECK(mb->mMode == M4_MBMODE_INTER || mb->mMode == M4_MBMODE_INTER_Q || mb->mMode == M4_MBMODE_INTER4V );

	M4BitstreamCacheEntry* pCacheEntry = nullptr;

	// process coded macroblocks if we have some
	if (cbp)
	{
		pCacheEntry = &mBitstreamCache.Alloc();
		int16* pDctBitstream =  pCacheEntry->mDctFromBitstream;

		for(uint32 i=0; i<6; ++i)
		{
			// check if any block is coded inside stream
			if (cbp & (1 << (5-i)))
			{
				uint32 dctOffset = i<<6;
				mBitstreamParser.decodeInterBlock(pDctBitstream+dctOffset);
			}
		}
	}
	mXCommand.XUpdateInterMB(mb, mbx, mby, pCacheEntry, mvPrediction, refImgNo);
}


// ----------------------------------------------------------------------------
/**
 * Decode an INTER macroblock for B frames using interpolation
 *
 * @param refImgForward
 * @param refImgBackward
 * @param mb
 * @param mbx
 * @param mby
 * @param cbp
 * @param mbLastIdx
 */
void M4Decoder::mbInterpolate(uint32 refImgForward, uint32 refImgBackward, M4_MB* mb, int32 mbx, int32 mby, uint32 cbp, uint16 mbLastIdx)
{
	M4BitstreamCacheEntry* pCacheEntry = nullptr;

	if (cbp)
	{
		pCacheEntry = &mBitstreamCache.Alloc();
		int16* pDctBitstream =  pCacheEntry->mDctFromBitstream;

		for(uint32 i=0; i<6; ++i)
		{
			// check if any block is coded inside stream
			if (cbp & (1 << (5-i)))
			{
				uint32 dctOffset = i<<6;
				mBitstreamParser.decodeInterBlock(pDctBitstream+dctOffset);
			}
		}
	}
	mXCommand.XInterpolateMB(mb, mbx, mby, pCacheEntry, refImgForward, refImgBackward, mbLastIdx);
}



#ifdef _M4_ENABLE_BMP_OUT
// ----------------------------------------------------------------------------
/**
 * Handle settings for BMP output
 *
 * @param pFileBaseName
 */
void M4Decoder::SetVideoOutToBMP(const char* pFileBaseName)
{
	if (pFileBaseName)
	{
		strncpy(mBaseName, pFileBaseName, sizeof(mBaseName));
		mBaseName[sizeof(mBaseName)-1] = '\0';
	}
	else
	{
		mBaseName[0] = '\0';
	}
}
#endif

}

