// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"
#include "M4Global.h"
#include "M4BitstreamParser.h"
#include "M4Prediction.h"
#include "M4XCmdSingleThread.h"

namespace vdecmpeg4
{
class M4Image;
class M4MotionVectorMgr;
class VIDStream;

// Linear texture
#define M4_MEM_OFFSET_LEFT_BLOCK	8	// 8 bytes
#define M4_MEM_SHIFT_MB_TO_Y		4	// * 16
#define M4_MEM_SHIFT_MB_TO_UV		3	// * 8


/*!
 ******************************************************************************
 * Main decoder interface
 *
 * Note: This class is not designed for inheritance.
 ******************************************************************************
 */
class M4Decoder
{
public:
	//! Static creation
	static M4Decoder* create(M4MemHandler& memHandler);

	//! Static destruction
	static void destroy(M4Decoder*& pDecoder, M4MemHandler& memHandler);

	//! Initialize decoder
	VIDError init(const VIDDecoderSetup* setup);

	//! Init decoder for indicated image res
	VIDError initBuffers(int16 width, int16 height);

	//-------------------------------------------------------------------------
	//! "Pull" interface

	//! Attach a new stream interface
	VIDError StreamSet(VIDStreamIO* pStream, VIDStreamEvents* pEvents);

	//! Handle Stream
	VIDError StreamDecode(float time, const VIDImage** result);


	//-------------------------------------------------------------------------
	//! @name "Push" interface

	//! Attach a new event interface
	VIDError StreamEventsSet(VIDStreamEvents* pEvents)
	{
		mpStreamEvents = pEvents;
		return VID_OK;
	}




	//! Do some reset stuff if seeking happens
	VIDError StreamSeekNotify();

private:
	//! Default constructor
	M4Decoder(M4MemHandler& memHandler);

	//! Destructor
	~M4Decoder();

	//! Decode I-frame
	VIDError iFrame();

	//! Decode p-frame
	VIDError pFrame(bool gmc);

	//! Decode B-frame
	VIDError bFrame();

	//! Decode INTRA macroblock
	void mbIntra(M4_MB* mb, int32 x, int32 y, bool useACPredition, uint32 cbp, uint8 quant);

	//! Decode INTER macroblock
	void mbInter(M4_MB* mb, int32 x, int32 y, uint32 cbp, MV_PREDICTION mvPrediction, uint32 refImgNo);

	//! Decode INTER macroblock using interpolation for B frames
	void mbInterpolate(uint32 refImgForward, uint32 refImgBackward, M4_MB* mb, int32 mbx, int32 mby, uint32 cbp, uint16 mbLastIdx = 0);

	//! Block usage of default constructor
	M4Decoder();

	//! Copy-constructor is private to prevent usage!
	M4Decoder(const M4Decoder& pObj);

	//! Helper to free resources
	void freeStuff();

	//! Just release buffer stuff
	void freeBuffers();

	//! Try to allocate a free frame buffer
	M4Image* AllocVidFrame();

	//! Assignment operator is private to prevent usage!
	const M4Decoder& operator=(const M4Decoder& pObj);

private:
	int16					mWidth;			   	 	//!< original image width in pixel
	int16					mHeight;		   		//!< original image height in pixel

	uint16					mMBWidth;				//!< # of horizonal 16x16 macro blocks
	uint16 					mMBHeight;				//!< # of vertical 16x16 macro blocks

	uint32					mDecoderFlags;			//!< decoder creation flags and modes

	M4Image*				mCurrent;

	//! 0: last I or P for backward prediction
	//! 1: forward reference VOP (most recently decoded VOP in the past)
	M4Image*				mReference[2];

	//! mTempImage[0] is used for B frame interpolation
	//! mTempImage[1] is used as additional output buffer, because Flipper works asynchronously
	M4Image*				mTempImage[2];

	M4_MB* 					mIPMacroblocks;			//!< macroblocks for I- or P-Frame
	M4_MB* 					mBMacroblocks;			//!< macroblocks for B-Frame

	M4MotionVectorMgr*		mMotVecMgr;

	M4Bitstream				mBitstream;
	M4BitstreamParser 		mBitstreamParser;		//!< connection to our bitstream

	M4Image** 				mpVidImage;
	uint32					mNumVidImages;


	uint32					mFrameCounter;
	M4BitstreamCache		mBitstreamCache;

	VIDStreamIO* 			mpStreamIO;			//!< new stream interface
	VIDStreamEvents*		mpStreamEvents;

	M4XCmdSingleThread		mXCommand;

	//! 'Everyone needs some friends'
	friend class M4MotionVectorMgr;
	friend class M4Image;

public:

	void* operator new(size_t sz, M4MemHandler& memSys)
	{
		return memSys.malloc(sz);
	}
	void operator delete(void* ptr)
	{
		((M4Decoder*)ptr)->mMemSys.free(ptr);
	}
	static const int32 		mDequantTable[4];
	static const uint32 	mRoundtab[16];
	static const uint8 		mIntraMacroBlocks[8];

	//! local copy of memory hooks
	M4MemHandler mMemSys;

	//! Install memory handling code
	M4_MEMORY_HANDLER

	//! Required stuff for debugging BMP output
#ifdef _M4_ENABLE_BMP_OUT
	void SetVideoOutToBMP(const char* pFileBaseName);
	char mBaseName[256];
#endif

	friend class M4XCmdSingleThread;	// temp hack

	void (*mCbPrintf)(const char *pMessage);
	void VIDPrintf(const char *pFormat, ...);
};

}

