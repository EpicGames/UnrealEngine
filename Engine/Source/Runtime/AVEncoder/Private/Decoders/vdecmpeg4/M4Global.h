// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4_Types.h"

namespace vdecmpeg4
{

// --------------------------------------------------------------------------------

//! supported macroblock coding types
enum M4_MBMODE
{
	M4_MBMODE_INTER			 	= 0,
    M4_MBMODE_INTER_Q		 	= 1,
	M4_MBMODE_INTER4V	   	 	= 2,
	M4_MBMODE_INTRA			 	= 3,
    M4_MBMODE_INTRA_Q		 	= 4,
	M4_MBMODE_STUFFING		 	= 7,
	M4_MBMODE_NOT_CODED_PVOP 	= 16,
	M4_MBMODE_NOT_CODED_SVOP 	= 17
};

//! macroblock additional flags
enum M4_MBFLAGS
{
	M4_MBFLAG_MVTYPE_DIRECT	 		 = 0,		// 00
    M4_MBFLAG_MVTYPE_INTERPOLATE	 = 1,		// 01
	M4_MBFLAG_MVTYPE_BACKWARD		 = 2,		// 10
	M4_MBFLAG_MVTYPE_FORWARD		 = 3,		// 11
	M4_MBFLAG_MVTYPE_MASK	 		 = 0x03,

	M4_MBFLAG_FIELD_DCT_BIT			 = 0x04,
	M4_MBFLAG_FIELD_PREDICTION_BIT	 = 0x08,

	M4_MBFLAG_FIELD_REF_FWD_MASK	 = 0x30,	// we use 2 bits storing the latest ref_field_top/bottom
	M4_MBFLAG_HAS_MOTION_VECTOR		 = 0x80
};

#ifndef M4MAX
#define M4MAX(A,B)		((A)>(B)?(A):(B))
#endif

#ifndef M4MIN
#define M4MIN(A,B)		((A)<(B)?(A):(B))
#endif

#define M4_MBPRED_SIZE  15



//! Information per macroblock filled-in by the bitstream parser
struct M4BitstreamCacheEntry
{
/*  4*/	uint32	mState;						//!< a value != 0 means that this entry is currently used
/* 12*/	uint16	mDcScaler[6];				//!< temporary calculated dc prediction values
/*768*/	int16	mDctFromBitstream[6*64];	//!< parsed idct coefficients from the bitstream (6 blocks (4:1:1) by 8x8 values)
};

//! Possible motion vector prediction
enum MV_PREDICTION
{
	MV_PREDICTION_NONE,
	MV_PREDICTION_P_FORWARD,
	MV_PREDICTION_B_FORWARD,
	MV_PREDICTION_B_BACKWARD
};

//! Motion vector
struct M4_VECTOR
{
	int32 x;
	int32 y;
};


// ------------------------------------------------------------------------------
// Macroblock structure
//
// There a sizeof check somewhere - dont enlarge this struct too much
//
// Currently, it 256 bytes. Having a 1920x1088p frame means
//
//	!!	120 macroblocks per row x 68 rows = 8160 macroblocks !!
//  !!  and this yields around 2 MB of macroblock only data  !!
// ------------------------------------------------------------------------------

struct M4_MB
{
	uint8			mMode 	   : 7;								    // 0
	uint8			mModeIntra : 1;

	uint8			mQuant 		: 5;								// 1
	uint8			mMvResidual : 1;
	uint8			_free 		: 2;

	uint8			mACPredDirections[6];							// 2 + 6
	M4_VECTOR 		mFMv[4];		//!< forward motion vectors		// 8 + (4*8)
	int16 			mPredictedValues[6][M4_MBPRED_SIZE];			// 40
	uint8			mFlags;
	uint8			mCbp;
	M4_VECTOR 		mBMv[4];		//!< backward motion vectors
};


}

