// Copyright Epic Games, Inc. All Rights Reserved.
#include "M4Image.h"
#include "vdecmpeg4.h"


namespace vdecmpeg4
{
#ifdef _M4_ENABLE_BMP_OUT

// the following defines are ONLY used for the yuv-to-rgb conversion
#define RGB_Y_OUT		1.164
#define B_U_OUT			2.018
#define Y_ADD_OUT		16
#define G_U_OUT			0.391
#define G_V_OUT			0.813
#define U_ADD_OUT		128
#define R_V_OUT			1.596
#define V_ADD_OUT		128
#define SCALEBITS_OUT	13
#define FIX_OUT(x)		((uint16)((x) * (1L<<SCALEBITS_OUT) + 0.5))

static int32 calcPixelOffset(int32 x, int32 y, int32 stride, int32 width, int32 height, bool isUV=false)
{
//	M4CHECKF(x > (isUV?-16:-32) && x < width+(isUV?16:32), TEXT("Access outside frame\n"));
//	M4CHECKF(y > (isUV?-16:-32) && y < height+(isUV?16:32), TEXT("Access outside frame\n"));

	int32 result = y * stride + x;

//	M4CHECKF(result > -stride*(isUV?16:32), TEXT("Memory offset outside frame\n"));
//	M4CHECKF(result < stride*(height+(isUV?16:32)), TEXT("Memory offset outside frame\n"));

	return result;
}

#define SET_LE_32(ptr, value)								\
	*(ptr) = (char)(((uint32)value) & 0xff);				\
	*((ptr)+1) = (char)((((uint32)value)>>8) & 0xff);		\
	*((ptr)+2) = (char)((((uint32)value)>>16) & 0xff);	\
	*((ptr)+3) = (char)((((uint32)value)>>24) & 0xff)

#define SET_LE_16(ptr, value)				\
	*(ptr) = (value) & 0xff;				\
	*((ptr)+1) = ((value)>>8) & 0xff


int32 	M4Image::mTabRgbY[256];
int32 	M4Image::mTabBU[256];
int32 	M4Image::mTabGU[256];
int32 	M4Image::mTabGV[256];
int32 	M4Image::mTabRV[256];
bool 	M4Image::mInitImageSave = false;
#endif


// ----------------------------------------------------------------------------
/**
 * Release the image from its usage (usually after GPU is finished)
 */
void VIDImage::Release() const
{
	M4Image *pImage = (M4Image*)_private;
	pImage->RefRemove();
}


// ----------------------------------------------------------------------------
/**
 * Constructor for image buffer.
 *
 * The provided image buffer is 'extended' with a padding area around
 * all borders. Currently, this padding area is 32 pixel and this size
 * is FIXED, because some assembler routines just ASSUME this size.
 *
 * @param decoder
 * @param width   original image width
 * @param height  original image height
 *
 * @return
 */
M4Image* M4Image::create(M4Decoder* decoder, int16 width, int16 height, bool /*poolMem*/)
{
	M4Image* img = new(decoder->mMemSys) M4Image;
	if (img == nullptr)
	{
		return nullptr;
	}

	M4CHECK(decoder);
	img->mDecoder = decoder;

	// Newly allocated image means at least referenced once
	img->mRefCount = 1;

	img->mImage._private = img;
	img->mImage.width  = width;
	img->mImage.height = height;


	// Align to multiple of 16 and add border left/right (or top/bottom
	img->mImage.texWidth  = ((width +15)&~15) + 2*EDGE_SIZE;		// add 32 pixel on left and right side
	img->mImage.texHeight = ((height+15)&~15) + 2*EDGE_SIZE;		// add 32 lines on top and bottom side

	int32 memReq = img->mImage.texWidth*img->mImage.texHeight + ALIGNMENT_TRASH
					 + 2*(((img->mImage.texWidth*img->mImage.texHeight)>>2) + ALIGNMENT_TRASH);

	img->mBaseMem = (uint8*)img->mDecoder->mMemSys.malloc((uint32) memReq, 64);
	M4CHECK((uint64(img->mBaseMem)&0x3f) == 0);
	if (img->mBaseMem == nullptr)
	{
		delete img;
		return nullptr;
	}
	img->setup(img->mBaseMem, 0);
	return img;
}


// ----------------------------------------------------------------------------
/**
 * Static destruction
 *
 * @param pImage
 */
void M4Image::destroy(M4Image*& pImage, M4Decoder*)
{
	if (pImage)
	{
		pImage->RefRemove();
		pImage = nullptr;
	}
}


// ----------------------------------------------------------------------------
/**
 * Destructor (private) - Need to call static destroy() method
 */
M4Image::~M4Image()
{
    if (mBaseMem)
	{
		mDecoder->mMemSys.free(mBaseMem);
		mBaseMem = nullptr;
	}
}


// ----------------------------------------------------------------------------
/**
 * Set buffer to black
 */
void M4Image::Black()
{
	M4CHECK(mImage.texture.y && mImage.texture.u && mImage.texture.v);
	M4CHECK(mImage.texWidth > 0 && mImage.texHeight > 0);

	FMemory::Memset(mImage.texture.y, 16,  (size_t)( (size_t)mImage.texWidth*(size_t)mImage.texHeight));
	FMemory::Memset(mImage.texture.u, 128, (size_t)(((size_t)mImage.texWidth*(size_t)mImage.texHeight)>>2));
	FMemory::Memset(mImage.texture.v, 128, (size_t)(((size_t)mImage.texWidth*(size_t)mImage.texHeight)>>2));
}


// ----------------------------------------------------------------------------
/**
 * Static callback for pool memory allocation
 *
 * @param memPtr
 * @param poolFlags
 * @param userData
 */
void M4Image::poolMemCb(void* memPtr, uint32, uint32 poolFlags, void* userData)
{
	M4Image* image = (M4Image *)userData;
	image->setup((uint8*)memPtr, poolFlags);
}


// ----------------------------------------------------------------------------
/**
 * Setup memory structures
 *
 * @param ptr
 */
void M4Image::setup(uint8* ptr, uint32)
{
	mImage.texture.y = ptr;
	mImage.texture.u = mImage.texture.y +	mImage.texWidth*mImage.texHeight	  + ALIGNMENT_TRASH;
	mImage.texture.v = mImage.texture.u + ((mImage.texWidth*mImage.texHeight)>>2) + ALIGNMENT_TRASH;

	M4CHECKF((((size_t)mImage.texture.y)&0x3f)==0, TEXT("mImage.texture.y not 64b aligned\n"));
	M4CHECKF((((size_t)mImage.texture.u)&0x3f)==0, TEXT("mImage.texture.u not 64b aligned\n"));
	M4CHECKF((((size_t)mImage.texture.v)&0x3f)==0, TEXT("mImage.texture.v not 64b aligned\n"));

    // calc start of image inside 'edged image' by adding border parameters
	mImage.y = mImage.texture.y + EDGE_SIZE  *  mImage.texWidth     + EDGE_SIZE;		// 32 lines top and 32 pixel left border
	mImage.u = mImage.texture.u + EDGE_SIZE2 * (mImage.texWidth>>1) + EDGE_SIZE2;	 	// 16 lines top and 16 pixel left border
	mImage.v = mImage.texture.v + EDGE_SIZE2 * (mImage.texWidth>>1) + EDGE_SIZE2; 		// 16 lines top and 16 pixel left border

	// set frame buffer to 'black' color
	Black();
}


#ifdef _M4_ENABLE_BMP_OUT

// ----------------------------------------------------------------------------
/**
 * Write image as bmp file back to host.
 *
 * This method is used for debugging purposes and writes the current
 * image as a .bmp file to the host.
 *
 * @param pBaseName
 * @param frameCounter
 */
void M4Image::saveBMP(const char* pBaseName, uint32 frameCounter)
{
	if (pBaseName == nullptr)
	{
		M4CHECK(false && "No basename specified");
		return;
	}

	int32 size = mImage.height * mImage.width * 3;
	char* rgb = (char*)mDecoder->mMemSys.malloc((size_t) size);
	getRGB((uint8*)rgb, mImage.width);

    // header of bmp file
	char header[14]={ 'B', 'M', 			// short int type
				    0, 0, 0, 0,			// int size: File size in bytes
					0, 0, 0, 0,			// short int reserved1, reserved2
				    14+40, 0, 0, 0 };  	// int offset: Offset to image data in bytes

	SET_LE_32(header+2, size + 14 + 40);	// filesize

	char infoHeader[40];
	SET_LE_32(infoHeader, 40);
	SET_LE_32(infoHeader+4, mImage.width);
	SET_LE_32(infoHeader+8, mImage.height);
	SET_LE_16(infoHeader+12, 1);	// Number of colour planes
	SET_LE_16(infoHeader+14, 24);	// Bits per pixel
	SET_LE_32(infoHeader+16, 0);	// Compression type NONE
	SET_LE_32(infoHeader+20, 0);	// Image size in bytes
	SET_LE_32(infoHeader+24, 0);	// x pixels per meter
	SET_LE_32(infoHeader+28, 0);	// y pixels per meter
	SET_LE_32(infoHeader+32, 0);	// number of colours in color idx (if any)
	SET_LE_32(infoHeader+36, 0);	// important colors

	char fName[256+8];
	snprintf(fName, sizeof(fName), "%s\\decoded_%05d.bmp", pBaseName, frameCounter);
	FILE* handle = fopen(fName, "wb");
	if (handle)
	{
		printf("M4Image::saveBMP: Saving: '%s'...\n", fName);
		if (fwrite(header, 1, 14, handle) != 14)
		{
			printf("M4Image::saveBMP: Write error on file: '%s'\n", fName);
		}
		if (fwrite(infoHeader, 1, 40, hande) != 40)
		{
			printf("M4Image::saveBMP: Write error on file: '%s'\n", fName);
		}
		if (fwrite(rgb, 1, (size_t)size, handle) != size)
		{
			printf("M4Image::saveBMP: Write error on file: '%s'\n", fName);
		}
		fclose(handle);
	}
	else
	{
		printf("M4Image::saveBMP: Cannot open file: '%s'\n", fName);
	}

	mDecoder->mMemSys.free(rgb);
}


// ----------------------------------------------------------------------------
/**
 * yuv 4:2:0 to rgb24
 *
 * Provided destination memory must be big enough!
 *
 * @param dst       pointer to destination memory
 * @param dstStride width of one row in destination memory
 */
void M4Image::getRGB(uint8* dst, int32 dstStride)
{
	// make some local paramers
	int32 width = mImage.width;
	int32 height = mImage.height;

	int32 yStride = mImage.texWidth;
	int32 uvStride = yStride/2;

	initImageSave();

	const int32 dstDif = 2 * 3 * dstStride + 3 * width;

	dst += 3 * dstStride * (mImage.height-1);
	uint8 *dst2 = dst - 3 * dstStride;

	for(int32 y=0; y<height/2; ++y)
	{
		for(int32 x=0; x<width/2; ++x)
		{
			int32 b_u, g_uv, r_v, rgb_y;
			int32 r, g, b;

			// Set to #if 0 to write Luma only
#if 1
			uint8 u = mImage.u[calcPixelOffset(x,y,uvStride,width,height,true)];
			uint8 v = mImage.v[calcPixelOffset(x,y,uvStride,width,height,true)];
#else
			uint8 u = 128;
			uint8 v = 128;
#endif
			b_u = mTabBU[u];
			g_uv = mTabGU[u] + mTabGV[v];
			r_v = mTabRV[v];

			rgb_y = mTabRgbY[ mImage.y[calcPixelOffset(x*2,y*2,yStride,width,height)] ];
			b = (rgb_y + b_u) >> SCALEBITS_OUT;
			g = (rgb_y - g_uv) >> SCALEBITS_OUT;
			r = (rgb_y + r_v) >> SCALEBITS_OUT;
			dst[0] = (uint8)M4MAX(0,M4MIN(255, b));
			dst[1] = (uint8)M4MAX(0,M4MIN(255, g));
			dst[2] = (uint8)M4MAX(0,M4MIN(255, r));

			rgb_y = mTabRgbY[  mImage.y[calcPixelOffset(x*2+1,y*2,yStride,width,height)] ];
			b = (rgb_y + b_u) >> SCALEBITS_OUT;
			g = (rgb_y - g_uv) >> SCALEBITS_OUT;
			r = (rgb_y + r_v) >> SCALEBITS_OUT;
			dst[3] = (uint8)M4MAX(0,M4MIN(255, b));
			dst[4] = (uint8)M4MAX(0,M4MIN(255, g));
			dst[5] = (uint8)M4MAX(0,M4MIN(255, r));

			rgb_y = mTabRgbY[  mImage.y[calcPixelOffset(x*2,y*2+1,yStride,width,height)] ];
			b = (rgb_y + b_u) >> SCALEBITS_OUT;
			g = (rgb_y - g_uv) >> SCALEBITS_OUT;
			r = (rgb_y + r_v) >> SCALEBITS_OUT;
			dst2[0] = (uint8)M4MAX(0,M4MIN(255, b));
			dst2[1] = (uint8)M4MAX(0,M4MIN(255, g));
			dst2[2] = (uint8)M4MAX(0,M4MIN(255, r));

			rgb_y = mTabRgbY[  mImage.y[calcPixelOffset(x*2+1,y*2+1,yStride,width,height)] ];
			b = (rgb_y + b_u) >> SCALEBITS_OUT;
			g = (rgb_y - g_uv) >> SCALEBITS_OUT;
			r = (rgb_y + r_v) >> SCALEBITS_OUT;
			dst2[3] = (uint8)M4MAX(0,M4MIN(255, b));
			dst2[4] = (uint8)M4MAX(0,M4MIN(255, g));
			dst2[5] = (uint8)M4MAX(0,M4MIN(255, r));

			dst += 6;
			dst2 += 6;
		}
		dst -= dstDif;
		dst2 -= dstDif;
	}
}


// ----------------------------------------------------------------------------
/**
 * init yuv-to-rgb colorspace tables
 */
void M4Image::initImageSave()
{
	if (!mInitImageSave)
	{
		mInitImageSave = true;
		for(uint32 i=0; i<256; ++i)
		{
			mTabRgbY[i] = (int32)(FIX_OUT(RGB_Y_OUT) * (i - Y_ADD_OUT));
			mTabBU[i] 	= (int32)(FIX_OUT(B_U_OUT) * (i - U_ADD_OUT));
			mTabGU[i] 	= (int32)(FIX_OUT(G_U_OUT) * (i - U_ADD_OUT));
			mTabGV[i] 	= (int32)(FIX_OUT(G_V_OUT) * (i - V_ADD_OUT));
			mTabRV[i] 	= (int32)(FIX_OUT(R_V_OUT) * (i - V_ADD_OUT));
		}
	}
}
#endif



// ----------------------------------------------------------------------------
/**
 * Create padding
 *
 * @param img
 */
void M4ImageCreatePadding(void* img)
{
	const M4Image* that = (const M4Image*)img;
	//
	// create Y padding
	//
	uint8* dst = that->mImage.texture.y;
	uint8* src = that->mImage.y;

	// fill 32 top padding lines
	for(uint16 i=0; i<EDGE_SIZE; ++i)
	{
		FMemory::Memset(dst, *src, EDGE_SIZE);
		FMemory::Memcpy(dst + EDGE_SIZE, src, (size_t)that->mImage.width);
		FMemory::Memset(dst + EDGE_SIZE + that->mImage.width, *(src + that->mImage.width - 1), EDGE_SIZE);
		dst += that->mImage.texWidth;
	}

	// fill left and right 'outside' padding area
	for(uint16 i=0; i<that->mImage.height; ++i)
	{
		FMemory::Memset(dst, *src, EDGE_SIZE);
		FMemory::Memset(dst + EDGE_SIZE + that->mImage.width, *(src + that->mImage.width - 1), EDGE_SIZE);
		dst += that->mImage.texWidth;
		src += that->mImage.texWidth;
	}

	// fill 32 bottom padding lines
	src -= that->mImage.texWidth;					// go back to last row with data
    for(uint16 i=0; i<EDGE_SIZE; ++i)
	{
		FMemory::Memset(dst, *src, EDGE_SIZE);
		FMemory::Memcpy(dst + EDGE_SIZE, src, (size_t)that->mImage.width);
		FMemory::Memset(dst + EDGE_SIZE + that->mImage.width, *(src + that->mImage.width - 1), EDGE_SIZE);
		dst += that->mImage.texWidth;
	}

	//
	// create u padding
	//

	// fill 16 top padding lines
	dst = that->mImage.texture.u;
    src = that->mImage.u;
    int32 width2 = that->mImage.width/2;
	int32 stride = that->mImage.texWidth/2;

    for(uint16 i=0; i<EDGE_SIZE2; ++i)
	{
		FMemory::Memset(dst, *src, EDGE_SIZE2);
		FMemory::Memcpy(dst + EDGE_SIZE2, src, (size_t)width2);
		FMemory::Memset(dst + EDGE_SIZE2 + width2, *(src + width2 - 1), EDGE_SIZE2);
		dst += stride;
	}

	// fill left and right 'outside' padding area
    for(uint16 i=0; i<(that->mImage.height/2); ++i)
	{
		FMemory::Memset(dst, *src, EDGE_SIZE2);
		FMemory::Memset(dst + EDGE_SIZE2 + width2, *(src + width2 - 1), EDGE_SIZE2);
		dst += stride;
		src += stride;
	}

	// fill 16 bottom padding lines
	src -= stride;
    for(uint16 i=0; i<EDGE_SIZE2; ++i)
	{
		FMemory::Memset(dst, *src, EDGE_SIZE2);
		FMemory::Memcpy(dst + EDGE_SIZE2, src, (size_t)width2);
		FMemory::Memset(dst + EDGE_SIZE2 + width2, *(src + width2 - 1), EDGE_SIZE2);
		dst += stride;
	}

	//
	// create V padding
	//

	// fill 16 top padding lines
	dst = that->mImage.texture.v;
    src = that->mImage.v;

    for(uint16 i=0; i<EDGE_SIZE2; ++i)
	{
		FMemory::Memset(dst, *src, EDGE_SIZE2);
		FMemory::Memcpy(dst + EDGE_SIZE2, src, (size_t)width2);
		FMemory::Memset(dst + EDGE_SIZE2 + width2, *(src + width2 - 1), EDGE_SIZE2);
		dst += stride;
	}

	// fill left and right 'outside' padding area
    for(uint16 i=0; i<(that->mImage.height/2); ++i)
	{
		FMemory::Memset(dst, *src, EDGE_SIZE2);
		FMemory::Memset(dst + EDGE_SIZE2 + width2, *(src + width2 - 1), EDGE_SIZE2);
		dst += stride;
		src += stride;
	}

	// fill 16 bottom padding lines
	src -= stride;
    for(uint16 i=0; i<EDGE_SIZE2; ++i)
	{
		FMemory::Memset(dst, *src, EDGE_SIZE2);
		FMemory::Memcpy(dst + EDGE_SIZE2, src, (size_t)width2);
		FMemory::Memset(dst + EDGE_SIZE2 + width2, *(src + width2 - 1), EDGE_SIZE2);
		dst += stride;
	}
}

}


