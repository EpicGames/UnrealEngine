// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"
#include "M4Decoder.h"

namespace vdecmpeg4
{

//! image buffer top, left, right and bottom edge size.
//! Must be 32 in this version.
#define EDGE_SIZE 32

//! image buffer edge size for u and v buffer (half resolution)
#define EDGE_SIZE2	(EDGE_SIZE/2)

//! some additional bytes required for allocation alignment
#define ALIGNMENT_TRASH	(64)


class M4Image
{
public:
	//! original image (without edges)
	//! A pointer to this will be returned to the caller
	VIDImage 	mImage;

	//! Image information
	VIDImageInfo mImageInfo;

private:
	M4Image()
	{
	}

public:
	//
	//--- Public Interface
	//

	//! Construction and allocation of image helper
	static M4Image* create(M4Decoder* decoder, int16 width, int16 height, bool poolMem = true);

	//! Destruction
	static void destroy(M4Image*& pImage, M4Decoder* decoder);

	//! Clear buffer
	void Black();

	//! Reference increase
	void RefAdd()
	{
		++mRefCount;
	}

	//! Reference decrease and free mem if drops to 0 references
	void RefRemove()
	{
		if (--mRefCount == 0)
		{
			// Actually, we need to get a copy of the member variable, because the destructor call
			// in the next line frees the instance possibily overwriting the mDecoder member.
			M4Decoder* pDecoder = mDecoder;
			this->~M4Image();
			pDecoder->mMemSys.free(this);
		}
	}

	//! Return current reference value
	int32 RefGet() const
	{
		return mRefCount;
	}

	//! Return reference to our decoder
	M4Decoder& getDecoder() const
	{
		return *mDecoder;
	}

	//! Pool mem allocation callback
	static void poolMemCb(void* memPtr, uint32 size, uint32 poolFlags, void* userData);

	void* operator new(size_t sz, M4MemHandler& memSys)
	{
		void* pAddr = memSys.malloc(sz, 16);
		if (pAddr)
		{
			FMemory::Memzero(pAddr, sz);
		}
		return pAddr;
	}

	void operator delete(void* ptr)
	{
		((M4Image*)ptr)->mDecoder->mMemSys.free(ptr);
	}

private:
	//! Ptr to allocated private base mem
	uint8*			mBaseMem;

	//! Reference to our handling decoder
	M4Decoder*		mDecoder;

	//! Actual reference count
	volatile int32 mRefCount;


	//!	Destructor. Regular destruction happens from static method
	~M4Image();

	//! Perform memory ptr setup and initialization
	void setup(uint8* ptr, uint32 flags);

	//! Copy Constructor [not implemented]
	M4Image(const M4Image& pObj);

	//! Assignment operator [not implemented]
	const M4Image &operator=(const M4Image& pObj);

	//! Use special memory handling for this class
	M4_MEMORY_HANDLER

	friend class M4Decoder;
	friend struct VIDImage;


#ifdef _M4_ENABLE_BMP_OUT
	//! Save image to file on host
	void saveBMP(const char* pBaseName, uint32 frameCounter);

private:
	//! get image as bgr
	void getRGB(uint8* dst, int32 dstStride);

	//! init image save routines
	void initImageSave();

	static int32 mTabRgbY[256];
	static int32 mTabBU[256];
	static int32 mTabGU[256];
	static int32 mTabGV[256];
	static int32 mTabRV[256];
	static bool mInitImageSave;
#endif
};

/******************************************************************************
 *  INLINE FUNCTIONS
 ******************************************************************************/

inline void swapM4Image(M4Image*& img1, M4Image*& img2)
{
	M4Image* tmp = img1;
	img1 = img2;
	img2 = tmp;
}

}

