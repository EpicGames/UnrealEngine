// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"

namespace vdecmpeg4
{

void M4InvQuantType0Intra(int16* output, const int16* input, uint8 quantiserScale, uint16 DCScaler);
void M4InvQuantType0Inter(int16* output, const int16* input, uint8 quantiserScale);
void M4InvQuantType1Intra(int16* output, const int16* input, uint8 quantiserScale, uint16 DCScaler, const uint8* dequantMtx);
void M4InvQuantType1Inter(int16* output, const int16* input, uint8 quantiserScale, const uint8* dequantMtx);

}

