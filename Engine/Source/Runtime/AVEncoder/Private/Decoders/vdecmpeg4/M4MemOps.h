// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"

namespace vdecmpeg4
{

void M4MemOpIntraMBAll(void* mCurrent, int32 mbx, int32 mby, void* dct);
void M4MemOpInterMBCopyAll(void* mCurrent, int32 x, int32 y, void* mReference);
void M4MemOpInterMBAdd(void* mCurrent, int32 x, int32 y, void* dctData, uint32 cpb);
void M4MemHalfPelInterpolate(void* dst, void* src, int32 stride, int32 xpos, int32 ypos, void* mv, uint32 rounding, bool b4x4=false);
void M4MemOpInterpolateAll(void* mCurrent, int32 mbx, int32 mby, void* mReference);


class MemOpOffsets
{
public:
	MemOpOffsets() : mCurrentPitch(0)
	{}
	~MemOpOffsets() = default;

	void init(int32 pitch);

	uint16	blockIdxY[64];
	uint16	blockIdxUV[64];

	uint32	blockIdxY4[8];
	uint32	blockIdxUV4[8];

	int32	mCurrentPitch;
};


}

