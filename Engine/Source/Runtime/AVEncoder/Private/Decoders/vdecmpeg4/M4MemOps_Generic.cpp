// Copyright Epic Games, Inc. All Rights Reserved.

#include "M4Decoder.h"
#include "M4MemOps.h"
#include "M4Memory.h"
#include "M4Image.h"
#include "M4idct.h"
#include "M4InvQuant.h"



template <typename T, typename C>
T ADVANCE_POINTER(T pPointer, C numBytes)
{
	return T(size_t(pPointer) + size_t(numBytes));
}


namespace vdecmpeg4
{

// Enable code optimizations. Works only for little endian systems and may not result in more performant code than
// what the compiler generates, depending on the compiler.
// Some compilers produces better results without these bit fiddling optimizations.
#define ENABLE_OPTIMIZATION 0


static int16 iclip[1024];
static int16 *iclp = nullptr;

struct CLIPinitializer
{
	CLIPinitializer()
	{
		iclp = iclip + 512;
		for(int32 i= -512; i<512; i++)
		{
			iclp[i] = IntCastChecked<int16>(i < -256 ? -256 : i > 255 ? 255 : i);
		}
	}
};
static CLIPinitializer _sgClipTableInitializer;



static uint8 clampToUINT8(int16 In)
{
	return (uint8)(In < 0 ? 0 : In > 255 ? 255 : In);
}


static void _memTrans8to8Y(uint8* dst, const uint8* src, int32 stride)
{
	const uint64 * src64 = (const uint64 *)src;
		  uint64 * dst64 = 	   (uint64 *)dst;
	for(uint32 i=8; i; --i)
	{
		*dst64 = *src64;
		src64 = ADVANCE_POINTER(src64, stride);
		dst64 = ADVANCE_POINTER(dst64, stride);
	}
}

static void _memTrans8to8Yx4(uint8* dst, const uint8* src, int32 stride)
{
	const uint64 * src64 = (const uint64 *)src;
		  uint64 * dst64 = 	   (uint64 *)dst;
	for(uint32 i=16; i; --i)
	{
		dst64[0] = src64[0];
		dst64[1] = src64[1];
		src64 = ADVANCE_POINTER(src64, stride);
		dst64 = ADVANCE_POINTER(dst64, stride);
	}
}


static void _memTrans16to8Y(uint8* dst, const int16* src, const int32 stride)
{
	for(uint32 i=8; i; --i)
	{
		dst[0] = clampToUINT8(*src++);
		dst[1] = clampToUINT8(*src++);
		dst[2] = clampToUINT8(*src++);
		dst[3] = clampToUINT8(*src++);
		dst[4] = clampToUINT8(*src++);
		dst[5] = clampToUINT8(*src++);
		dst[6] = clampToUINT8(*src++);
		dst[7] = clampToUINT8(*src++);
		dst += stride;
	}
}

static void _memTrans16to8Yx4(uint8* dst, const int16* src, const int32 stride)
{
	const int16 *srcA = src;
	const int16 *srcB = src + 64;
	for(uint32 i=8; i; --i)
	{
		dst[ 0] = clampToUINT8(*srcA++);
		dst[ 1] = clampToUINT8(*srcA++);
		dst[ 2] = clampToUINT8(*srcA++);
		dst[ 3] = clampToUINT8(*srcA++);
		dst[ 4] = clampToUINT8(*srcA++);
		dst[ 5] = clampToUINT8(*srcA++);
		dst[ 6] = clampToUINT8(*srcA++);
		dst[ 7] = clampToUINT8(*srcA++);

		dst[ 8] = clampToUINT8(*srcB++);
		dst[ 9] = clampToUINT8(*srcB++);
		dst[10] = clampToUINT8(*srcB++);
		dst[11] = clampToUINT8(*srcB++);
		dst[12] = clampToUINT8(*srcB++);
		dst[13] = clampToUINT8(*srcB++);
		dst[14] = clampToUINT8(*srcB++);
		dst[15] = clampToUINT8(*srcB++);
		dst += stride;
	}

	srcA = srcB + 64;
	for(uint32 i=8; i; --i)
	{
		dst[ 0] = clampToUINT8(*srcB++);
		dst[ 1] = clampToUINT8(*srcB++);
		dst[ 2] = clampToUINT8(*srcB++);
		dst[ 3] = clampToUINT8(*srcB++);
		dst[ 4] = clampToUINT8(*srcB++);
		dst[ 5] = clampToUINT8(*srcB++);
		dst[ 6] = clampToUINT8(*srcB++);
		dst[ 7] = clampToUINT8(*srcB++);

		dst[ 8] = clampToUINT8(*srcA++);
		dst[ 9] = clampToUINT8(*srcA++);
		dst[10] = clampToUINT8(*srcA++);
		dst[11] = clampToUINT8(*srcA++);
		dst[12] = clampToUINT8(*srcA++);
		dst[13] = clampToUINT8(*srcA++);
		dst[14] = clampToUINT8(*srcA++);
		dst[15] = clampToUINT8(*srcA++);
		dst += stride;
	}
}


static void _memTrans16to8AddY(uint8* dst, const int16* src, const int32 stride)
{
	for(uint32 i=8; i; --i)
	{
		dst[0] = clampToUINT8((int16)dst[0] + *src++);
		dst[1] = clampToUINT8((int16)dst[1] + *src++);
		dst[2] = clampToUINT8((int16)dst[2] + *src++);
		dst[3] = clampToUINT8((int16)dst[3] + *src++);
		dst[4] = clampToUINT8((int16)dst[4] + *src++);
		dst[5] = clampToUINT8((int16)dst[5] + *src++);
		dst[6] = clampToUINT8((int16)dst[6] + *src++);
		dst[7] = clampToUINT8((int16)dst[7] + *src++);
		dst += stride;
	}
}


static void _memTrans16to8AddYx4(uint8* dst, const int16* src, const int32 stride)
{
	const int16 *srcA = src;
	const int16 *srcB = src + 64;
	for(uint32 i=8; i; --i)
	{
		dst[ 0] = clampToUINT8((int16)dst[ 0] + *srcA++);
		dst[ 1] = clampToUINT8((int16)dst[ 1] + *srcA++);
		dst[ 2] = clampToUINT8((int16)dst[ 2] + *srcA++);
		dst[ 3] = clampToUINT8((int16)dst[ 3] + *srcA++);
		dst[ 4] = clampToUINT8((int16)dst[ 4] + *srcA++);
		dst[ 5] = clampToUINT8((int16)dst[ 5] + *srcA++);
		dst[ 6] = clampToUINT8((int16)dst[ 6] + *srcA++);
		dst[ 7] = clampToUINT8((int16)dst[ 7] + *srcA++);

		dst[ 8] = clampToUINT8((int16)dst[ 8] + *srcB++);
		dst[ 9] = clampToUINT8((int16)dst[ 9] + *srcB++);
		dst[10] = clampToUINT8((int16)dst[10] + *srcB++);
		dst[11] = clampToUINT8((int16)dst[11] + *srcB++);
		dst[12] = clampToUINT8((int16)dst[12] + *srcB++);
		dst[13] = clampToUINT8((int16)dst[13] + *srcB++);
		dst[14] = clampToUINT8((int16)dst[14] + *srcB++);
		dst[15] = clampToUINT8((int16)dst[15] + *srcB++);
		dst += stride;
	}

	srcA = srcB + 64;
	for(uint32 i=8; i; --i)
	{
		dst[ 0] = clampToUINT8((int16)dst[ 0] + *srcB++);
		dst[ 1] = clampToUINT8((int16)dst[ 1] + *srcB++);
		dst[ 2] = clampToUINT8((int16)dst[ 2] + *srcB++);
		dst[ 3] = clampToUINT8((int16)dst[ 3] + *srcB++);
		dst[ 4] = clampToUINT8((int16)dst[ 4] + *srcB++);
		dst[ 5] = clampToUINT8((int16)dst[ 5] + *srcB++);
		dst[ 6] = clampToUINT8((int16)dst[ 6] + *srcB++);
		dst[ 7] = clampToUINT8((int16)dst[ 7] + *srcB++);

		dst[ 8] = clampToUINT8((int16)dst[ 8] + *srcA++);
		dst[ 9] = clampToUINT8((int16)dst[ 9] + *srcA++);
		dst[10] = clampToUINT8((int16)dst[10] + *srcA++);
		dst[11] = clampToUINT8((int16)dst[11] + *srcA++);
		dst[12] = clampToUINT8((int16)dst[12] + *srcA++);
		dst[13] = clampToUINT8((int16)dst[13] + *srcA++);
		dst[14] = clampToUINT8((int16)dst[14] + *srcA++);
		dst[15] = clampToUINT8((int16)dst[15] + *srcA++);
		dst += stride;
	}
}




void M4MemOpInterMBCopyAll(void* _current, int32 mbx, int32 mby, void* _reference)
{
    M4Image* current = (M4Image*)_current;
	M4Image* reference = (M4Image*)_reference;

	uint8* currentY   = current->mImage.y;
	uint8* referenceY = reference->mImage.y;
	int32 edgedWidth = current->mImage.texWidth;

	int32 stride = current->mImage.texWidth;

	static_assert(M4_MEM_SHIFT_MB_TO_Y == M4_MEM_SHIFT_MB_TO_UV+1, "Constant mismatch");
	static_assert(M4_MEM_OFFSET_LEFT_BLOCK == 8, "Constant mismatch");
	int32 commonOffset   = ( mbx       + edgedWidth * mby) << 4;
	int32 commonOffsetUV = ((mbx << 1) + edgedWidth * mby) << 2;

#if 1
	_memTrans8to8Yx4(currentY + commonOffset,                                                referenceY + commonOffset,                                                stride);
#else
	_memTrans8to8Y(currentY + commonOffset,                                                referenceY + commonOffset,                                                stride);
	_memTrans8to8Y(currentY + commonOffset + M4_MEM_OFFSET_LEFT_BLOCK,                     referenceY + commonOffset + M4_MEM_OFFSET_LEFT_BLOCK,                     stride);
	_memTrans8to8Y(currentY + commonOffset                            + (edgedWidth << 3), referenceY + commonOffset                            + (edgedWidth << 3), stride);
	_memTrans8to8Y(currentY + commonOffset + M4_MEM_OFFSET_LEFT_BLOCK + (edgedWidth << 3), referenceY + commonOffset + M4_MEM_OFFSET_LEFT_BLOCK + (edgedWidth << 3), stride);
#endif
	_memTrans8to8Y(current->mImage.u + commonOffsetUV, reference->mImage.u + commonOffsetUV, stride >> 1);
	_memTrans8to8Y(current->mImage.v + commonOffsetUV, reference->mImage.v + commonOffsetUV, stride >> 1);
}



void M4MemOpIntraMBAll(void* _current, int32 mbx, int32 mby, void* _dct)
{
	M4Image* current = (M4Image*)_current;
	const int16* dct = (const int16*)_dct;

	int32 stride = current->mImage.texWidth;
	int32 stride2 = stride >> 1;
	uint8* pY_Cur = current->mImage.y + (mby << 4) * stride  + (mbx << M4_MEM_SHIFT_MB_TO_Y);
	uint8* pU_Cur = current->mImage.u + (mby << 3) * stride2 + (mbx << M4_MEM_SHIFT_MB_TO_UV);
	uint8* pV_Cur = current->mImage.v + (mby << 3) * stride2 + (mbx << M4_MEM_SHIFT_MB_TO_UV);

#if 1
	_memTrans16to8Yx4(pY_Cur,										dct, stride);
#else
	int32 next_block = stride << 3;
	_memTrans16to8Y(pY_Cur,											&dct[0 * 64], stride);
	_memTrans16to8Y(pY_Cur + M4_MEM_OFFSET_LEFT_BLOCK,				&dct[1 * 64], stride);
	_memTrans16to8Y(pY_Cur + next_block,							&dct[2 * 64], stride);
	_memTrans16to8Y(pY_Cur + M4_MEM_OFFSET_LEFT_BLOCK + next_block, &dct[3 * 64], stride);
#endif
	_memTrans16to8Y(pU_Cur,											&dct[4 * 64], stride2);
	_memTrans16to8Y(pV_Cur,											&dct[5 * 64], stride2);
}



void M4InvQuantType0Intra(int16 *data, const int16 *coeff, uint8 quant, uint16 dcscalar)
{
#if 0
	const int32 quant_m_2 = quant << 1;
	const int32 quant_add = quant & ~1;

	data[0] = __ssat(coeff[0] * dcscalar, 12);
	for(uint32 i=63; i>0; --i)
	{
		int32 acLevel = coeff[i];
		if (acLevel == 0)
		{
			data[i] = 0;
		}
		else
		{
			data[i] = int16(__ssat(quant_m_2 * acLevel + (acLevel > 0 ? quant_add : -quant_add), 12));
		}
	}
#else
	const int32 quant_m_2 = quant << 1;
	const int32 quant_add = (quant & 1 ? quant : quant - 1);

	data[0] = coeff[0]  * dcscalar;

	if (data[0] < -2048)
	{
		data[0] = -2048;
	}
	else if (data[0] > 2047)
	{
		data[0] = 2047;
	}

	for(uint32 i=1; i<64; ++i)
	{
		int32 acLevel = coeff[i];

		if (acLevel == 0)
		{
			data[i] = 0;
		}
		else if (acLevel < 0)
		{
			acLevel = quant_m_2 * -acLevel + quant_add;
			data[i] = (int16)(acLevel <= 2048 ? -acLevel : -2048);
		}
		else
		{
			acLevel = quant_m_2 * acLevel + quant_add;
			data[i] = (int16)(acLevel <= 2047 ? acLevel : 2047);
		}
	}
#endif
}


void M4InvQuantType0Inter(int16 *data, const int16 *coeff, const uint8 quant)
{
	const uint16 quant_m_2 = (uint16)(quant << 1);
	const uint16 quant_add = (quant & 1 ? quant : quant - 1);

	for(uint32 i=0; i<64; ++i)
	{
		int16 acLevel = coeff[i];

		if (acLevel == 0)
		{
			data[i] = 0;
		}
		else if (acLevel < 0)
		{
			acLevel = acLevel * quant_m_2 - quant_add;
			data[i] = (acLevel >= -2048 ? acLevel : -2048);
		}
		else
		{
			acLevel = acLevel * quant_m_2 + quant_add;
			data[i] = (acLevel <= 2047 ? acLevel : 2047);
		}
	}
}



void M4MemOpInterMBAdd(void* _current, int32 mbx, int32 mby, void* _dct, uint32 cbp)
{
	M4Image* current = (M4Image*)_current;
	const int16* dct = (const int16*)_dct;

	int32 stride = current->mImage.texWidth;
	int32 stride2 = stride / 2;
	int32 next_block = stride * 8;
	uint8* pY_Cur = current->mImage.y + (mby << 4) * stride  + (mbx << M4_MEM_SHIFT_MB_TO_Y);
	uint8* pU_Cur = current->mImage.u + (mby << 3) * stride2 + (mbx << M4_MEM_SHIFT_MB_TO_UV);
	uint8* pV_Cur = current->mImage.v + (mby << 3) * stride2 + (mbx << M4_MEM_SHIFT_MB_TO_UV);

	if ((cbp & 60) == 60)
	{
		_memTrans16to8AddYx4(pY_Cur,                                         dct, stride);
	}
	else
	{
		if (cbp & 32) _memTrans16to8AddY(pY_Cur,                                         &dct[0 * 64], stride);
		if (cbp & 16) _memTrans16to8AddY(pY_Cur + M4_MEM_OFFSET_LEFT_BLOCK,              &dct[1 * 64], stride);
		if (cbp & 8)  _memTrans16to8AddY(pY_Cur + next_block,                            &dct[2 * 64], stride);
		if (cbp & 4)  _memTrans16to8AddY(pY_Cur + M4_MEM_OFFSET_LEFT_BLOCK + next_block, &dct[3 * 64], stride);
	}

	if (cbp & 2)  _memTrans16to8AddY(pU_Cur,                                         &dct[4 * 64], stride2);
	if (cbp & 1)  _memTrans16to8AddY(pV_Cur,                                         &dct[5 * 64], stride2);
}





void M4idct(int16* block)
{
	#define W1 2841 /* 2048*sqrt(2)*cos(1*pi/16) */
	#define W2 2676 /* 2048*sqrt(2)*cos(2*pi/16) */
	#define W3 2408 /* 2048*sqrt(2)*cos(3*pi/16) */
	#define W5 1609 /* 2048*sqrt(2)*cos(5*pi/16) */
	#define W6 1108 /* 2048*sqrt(2)*cos(6*pi/16) */
	#define W7 565  /* 2048*sqrt(2)*cos(7*pi/16) */

	int16* blk;
	int32 X0, X1, X2, X3, X4, X5, X6, X7;
	int32 tmp0;

	for(int32 i=7; i>=0; --i)	// idct rows
	{
		blk = block + (i << 3);
		X4 = blk[1];
		X3 = blk[2];
		X7 = blk[3];
		X1 = blk[4] << 11;
		X6 = blk[5];
		X2 = blk[6];
		X5 = blk[7];

		if (! (X1 | X2 | X3 | X4 | X5 | X6 | X7) )
		{
			blk[0] = blk[1] = blk[2] = blk[3] = blk[4] = blk[5] = blk[6] = blk[7] = (int16)(blk[0] << 3);
			continue;
		}

		X0 = (blk[0] << 11) + 128; 	// for proper rounding in the fourth stage

		// first stage
		tmp0 = W7 * (X4 + X5);
		X4 = tmp0 + (W1 - W7) * X4;
		X5 = tmp0 - (W1 + W7) * X5;

		tmp0 = W6 * (X3 + X2);
		X2 = tmp0 - (W2 + W6) * X2;
		X3 = tmp0 + (W2 - W6) * X3;

		tmp0 = W3 * (X6 + X7);
		X6 = tmp0 - (W3 - W5) * X6;
		X7 = tmp0 - (W3 + W5) * X7;

		// second stage

		tmp0 = X0 + X1;
		X0 -= X1;

		X1 = X4 + X6;
		X4 -= X6;
		X6 = X5 + X7;
		X5 -= X7;

		// third stage
		X7 = tmp0 + X3;
		tmp0 -= X3;
		X3 = X0 + X2;
		X0 -= X2;
		X2 = (181 * (X4 + X5) + 128) >> 8;
		X4 = (181 * (X4 - X5) + 128) >> 8;

		// fourth stage
		blk[0] = (int16)((X7 + X1) >> 8);
		blk[1] = (int16)((X3 + X2) >> 8);
		blk[2] = (int16)((X0 + X4) >> 8);
		blk[3] = (int16)((tmp0 + X6) >> 8);
		blk[4] = (int16)((tmp0 - X6) >> 8);
		blk[5] = (int16)((X0 - X4) >> 8);
		blk[6] = (int16)((X3 - X2) >> 8);
		blk[7] = (int16)((X7 - X1) >> 8);
	} // IDCT-rows

	for(int32 i=7; i>=0; --i)	// idct columns
	{
		blk = block + i;
		X1 = blk[8 * 4] << 8;
		X2 = blk[8 * 6];
		X3 = blk[8 * 2];
		X4 = blk[8 * 1];
		X5 = blk[8 * 7];
		X6 = blk[8 * 5];
		X7 = blk[8 * 3];

		if (! (X1 | X2 | X3 | X4 | X5 | X6 | X7) )
		{
			blk[8 * 0] = blk[8 * 1] = blk[8 * 2] = blk[8 * 3] = blk[8 * 4] =
			blk[8 * 5] = blk[8 * 6] = blk[8 * 7] = iclp[(blk[8 * 0] + 32) >> 6];
			continue;
		}

		X0 = (blk[8 * 0] << 8) + 8192;

		// first stage
		tmp0 = W7 * (X4 + X5) + 4;
		X4 = (tmp0 + (W1 - W7) * X4) >> 3;
		X5 = (tmp0 - (W1 + W7) * X5) >> 3;
		tmp0 = W3 * (X6 + X7) + 4;
		X6 = (tmp0 - (W3 - W5) * X6) >> 3;
		X7 = (tmp0 - (W3 + W5) * X7) >> 3;
		tmp0 = W6 * (X3 + X2) + 4;
		X2 = (tmp0 - (W2 + W6) * X2) >> 3;
		X3 = (tmp0 + (W2 - W6) * X3) >> 3;

		// second stage
		tmp0 = X0 + X1;
		X0 -= X1;
		X1 = X4 + X6;
		X4 -= X6;
		X6 = X5 + X7;
		X5 -= X7;

		// third stage
		X7 = tmp0 + X3;
		tmp0 -= X3;
		X3 = X0 + X2;
		X0 -= X2;
		X2 = (181 * (X4 + X5) + 128) >> 8;
		X4 = (181 * (X4 - X5) + 128) >> 8;

		// fourth stage
		blk[8 * 0] = iclp[(X7 + X1) >> 14];
		blk[8 * 1] = iclp[(X3 + X2) >> 14];
		blk[8 * 2] = iclp[(X0 + X4) >> 14];
		blk[8 * 3] = iclp[(tmp0 + X6) >> 14];
		blk[8 * 4] = iclp[(tmp0 - X6) >> 14];
		blk[8 * 5] = iclp[(X0 - X4) >> 14];
		blk[8 * 6] = iclp[(X3 - X2) >> 14];
		blk[8 * 7] = iclp[(X7 - X1) >> 14];
	}
}



static void _mbBlendSrcDst8x8(uint8* Dest, const uint8* Src, int32 Stride)
{
	const int32 blkSize = 8;
	for(int32 j=blkSize; j>0; --j)
	{
		const uint8* pS = Src;
			  uint8* pD = Dest;

		for(int32 i=blkSize; i>0; --i)
		{
			int32 tot = (*pS++ + *pD + 1) >> 1;
			*pD++ = (uint8)tot;
		}

		Src += Stride;
		Dest += Stride;
	}
}

static void _mbBlendSrcDst16x16(uint8* Dest, const uint8* Src, int32 Stride)
{
	const int32 blkSize = 16;
	for(int32 j=blkSize; j>0; --j)
	{
		const uint8* pS = Src;
			  uint8* pD = Dest;

		for(int32 i=blkSize; i>0; --i)
		{
			int32 tot = (*pS++ + *pD + 1) >> 1;
			*pD++ = (uint8)tot;
		}

		Src += Stride;
		Dest += Stride;
	}
}


static void _mbCopy8x8(uint8* Dest, const uint8* Src, int32 Stride)
{
	_memTrans8to8Y(Dest, Src, Stride);
}

static void _mbCopy16x16(uint8* Dest, const uint8* Src, int32 Stride)
{
	_memTrans8to8Yx4(Dest, Src, Stride);
}

static void _mbInterpolateHorizontal8x8(uint8* Dest, const uint8* Src, int32 Rounding, int32 Stride)
{
#if ENABLE_OPTIMIZATION
	uint64 R = 0;
	for(uint32 v=8; v; --v)
	{
			// BIG ENDIAN: uint64 A8 = *reinterpret_cast<const uint64*>(Src);
			// BIG ENDIAN: uint64 B8 = (A8 << 8) | Src[8];
	// NOTE: This is correct for little endian machines only
		uint64 A8 = *reinterpret_cast<const uint64*>(Src + 1);
		uint64 B8 = (A8 << 8) | Src[0];
		for(uint32 u=8; u; --u)
		{
			int32 V = ((A8 & 255) + (B8 & 255) + Rounding) >> 1;
			R = (R >> 8) | ((uint64)V << 56);
			A8 >>= 8;
			B8 >>= 8;
		}
		*reinterpret_cast<uint64*>(Dest) = R;
		Src  = ADVANCE_POINTER(Src,  Stride);
		Dest = ADVANCE_POINTER(Dest, Stride);
	}

#else
	const int32 blkSize = 8;
	for(int32 v=0; v<blkSize; ++v)
	{
		for(int32 u=0; u<blkSize; ++u)
		{
			int32 sum = (Src[u] + Src[u + 1] + Rounding) / 2;
			Dest[u] = (uint8)sum;
		}
		Src += Stride;
		Dest += Stride;
	}
#endif
}

static void _mbInterpolateHorizontal16x16(uint8* Dest, const uint8* Src, int32 Rounding, int32 Stride)
{
#if ENABLE_OPTIMIZATION
	uint64 R0 = 0;
	uint64 R1 = 0;
	for(uint32 v=16; v; --v)
	{
	// NOTE: This is correct for little endian machines only
		uint64 A8R = *reinterpret_cast<const uint64*>(Src + 1);
		uint64 A8L = *reinterpret_cast<const uint64*>(Src + 9);
		uint64 B8R = (A8R << 8) | Src[0];
		uint64 B8L = (A8L << 8) | (A8R >> 56);
		for(uint32 u=8; u; --u)
		{
			int32 V0 = ((A8R & 255) + (B8R & 255) + Rounding) >> 1;
			int32 V1 = ((A8L & 255) + (B8L & 255) + Rounding) >> 1;
			R0 = (R0 >> 8) | ((uint64)V0 << 56);
			R1 = (R1 >> 8) | ((uint64)V1 << 56);
			A8L >>= 8;
			A8R >>= 8;
			B8L >>= 8;
			B8R >>= 8;
		}
		*reinterpret_cast<uint64*>(Dest    ) = R0;
		*reinterpret_cast<uint64*>(Dest + 8) = R1;
		Src  = ADVANCE_POINTER(Src,  Stride);
		Dest = ADVANCE_POINTER(Dest, Stride);
	}

#else
	const int32 blkSize = 16;
	for(int32 v=0; v<blkSize; ++v)
	{
		for(int32 u=0; u<blkSize; ++u)
		{
			int32 sum = (Src[u] + Src[u + 1] + Rounding) / 2;
			M4CHECK(sum <= 255);
			Dest[u] = (uint8)sum;
		}
		Src += Stride;
		Dest += Stride;
	}
#endif
}

static void _mbInterpolateVertical8x8(uint8* Dest, const uint8* Src, int32 Rounding, int32 Stride)
{
#if ENABLE_OPTIMIZATION
	// NOTE: This is machine endian independent!
	uint64 R = 0;
	uint64 A8 = *reinterpret_cast<const uint64*>(Src);
	for(uint32 v=8; v; --v)
	{
		uint64 NextRow = *reinterpret_cast<const uint64*>(Src + Stride);
		uint64 B8 = NextRow;
		for(uint32 u=8; u; --u)
		{
			int32 V = ((A8 & 255) + (B8 & 255) + Rounding) >> 1;
			R = (R >> 8) | ((uint64)V << 56);
			A8 >>= 8;
			B8 >>= 8;
		}
		*reinterpret_cast<uint64*>(Dest) = R;
		A8 = NextRow;
		Src  = ADVANCE_POINTER(Src,  Stride);
		Dest = ADVANCE_POINTER(Dest, Stride);
	}

#else
	const int32 blkSize = 8;
	for(int32 v=0; v<blkSize; ++v)
	{
		for(int32 u=0; u<blkSize; ++u)
		{
			int32 sum = (Src[u] + Src[u + Stride] + Rounding) / 2;
			M4CHECK(sum <= 255);
			Dest[u] = (uint8)sum;
		}
		Src += Stride;
		Dest += Stride;
	}
#endif
}

static void _mbInterpolateVertical16x16(uint8* Dest, const uint8* Src, int32 Rounding, int32 Stride)
{
#if ENABLE_OPTIMIZATION
	// NOTE: This is machine endian independent!
	uint64 R0 = 0;
	uint64 R1 = 0;
	uint64 A8R = *reinterpret_cast<const uint64*>(Src);
	uint64 A8L = *reinterpret_cast<const uint64*>(Src + 8);
	for(uint32 v=16; v; --v)
	{
		uint64 NextRowR = *reinterpret_cast<const uint64*>(Src + Stride);
		uint64 NextRowL = *reinterpret_cast<const uint64*>(Src + Stride + 8);
		uint64 B8R = NextRowR;
		uint64 B8L = NextRowL;
		for(uint32 u=8; u; --u)
		{
			int32 V0 = ((A8R & 255) + (B8R & 255) + Rounding) >> 1;
			int32 V1 = ((A8L & 255) + (B8L & 255) + Rounding) >> 1;
			R0 = (R0 >> 8) | ((uint64)V0 << 56);
			R1 = (R1 >> 8) | ((uint64)V1 << 56);
			A8L >>= 8;
			A8R >>= 8;
			B8L >>= 8;
			B8R >>= 8;
		}
		*reinterpret_cast<uint64*>(Dest    ) = R0;
		*reinterpret_cast<uint64*>(Dest + 8) = R1;
		A8R = NextRowR;
		A8L = NextRowL;
		Src  = ADVANCE_POINTER(Src,  Stride);
		Dest = ADVANCE_POINTER(Dest, Stride);
	}

#else
	const int32 blkSize = 16;
	for(int32 v=0; v<blkSize; ++v)
	{
		for(int32 u=0; u<blkSize; ++u)
		{
			int32 sum = (Src[u] + Src[u + Stride] + Rounding) / 2;
			M4CHECK(sum <= 255);
			Dest[u] = (uint8)sum;
		}
		Src += Stride;
		Dest += Stride;
	}
#endif
}

static void _mbInterpolateBoth8x8(uint8* Dest, const uint8* Src, int32 Rounding, int32 Stride)
{
#if ENABLE_OPTIMIZATION
// NOTE: This is correct for little endian machines only
	uint64 R = 0;
	uint64 R0C0 = *reinterpret_cast<const uint64*>(Src + 1);
	uint64 R0C1 = (R0C0 << 8) | Src[0];
	Src  = ADVANCE_POINTER(Src,  Stride);
	for(uint32 v=8; v; --v)
	{
		uint64 NextRow0 = *reinterpret_cast<const uint64*>(Src + 1);
		uint64 NextRow1 = (NextRow0 << 8) | Src[0];
		uint64 R1C0 = NextRow0;
		uint64 R1C1 = NextRow1;
		for(uint32 u=8; u; --u)
		{
			int32 V = ((R0C0 & 255) + (R0C1 & 255) + (R1C0 & 255) + (R1C1 & 255) + Rounding) >> 2;
			R = (R >> 8) | ((uint64)V << 56);
			R0C0 >>= 8;
			R0C1 >>= 8;
			R1C0 >>= 8;
			R1C1 >>= 8;
		}
		*reinterpret_cast<uint64*>(Dest) = R;
		R0C0 = NextRow0;
		R0C1 = NextRow1;
		Src  = ADVANCE_POINTER(Src,  Stride);
		Dest = ADVANCE_POINTER(Dest, Stride);
	}

#else
	const int32 blkSize = 8;
	for(int32 v=0; v<blkSize; ++v)
	{
		for(int32 u=0; u<blkSize; ++u)
		{
			int32 sum = (Src[u] + Src[u + 1] + Src[u + Stride] + Src[u + Stride + 1] + Rounding) / 4;
			M4CHECK(sum <= 255);
			Dest[u] = (uint8)sum;
		}
		Src += Stride;
		Dest += Stride;
	}
#endif
}

static void _mbInterpolateBoth16x16(uint8* Dest, const uint8* Src, int32 Rounding, int32 Stride)
{
#if ENABLE_OPTIMIZATION
// NOTE: This is correct for little endian machines only
	uint64 R0 = 0;
	uint64 R1 = 0;
	uint64 R0C1 = *reinterpret_cast<const uint64*>(Src + 1);
	uint64 R0C2 = *reinterpret_cast<const uint64*>(Src + 9);
	uint64 R0C1S = (R0C1 << 8) | Src[0];
	uint64 R0C2S = (R0C2 << 8) | (R0C1 >> 56);
	Src  = ADVANCE_POINTER(Src,  Stride);
	for(uint32 v=16; v; --v)
	{
		uint64 R1C1 = *reinterpret_cast<const uint64*>(Src + 1);
		uint64 R1C2 = *reinterpret_cast<const uint64*>(Src + 9);
		uint64 R1C1S = (R1C1 << 8) | Src[0];
		uint64 R1C2S = (R1C2 << 8) | (R1C1 >> 56);
		uint64 n0 = R1C1;
		uint64 n1 = R1C2;
		uint64 n2 = R1C1S;
		uint64 n3 = R1C2S;
		for(uint32 u=8; u; --u)
		{
			int32 V0 = ((R0C1 & 255) + (R0C1S & 255) + (R1C1 & 255) + (R1C1S & 255) + Rounding) >> 2;
			int32 V1 = ((R0C2 & 255) + (R0C2S & 255) + (R1C2 & 255) + (R1C2S & 255) + Rounding) >> 2;
			R0 = (R0 >> 8) | ((uint64)V0 << 56);
			R1 = (R1 >> 8) | ((uint64)V1 << 56);
			R0C1  >>= 8;
			R0C2  >>= 8;
			R0C1S >>= 8;
			R0C2S >>= 8;
			R1C1  >>= 8;
			R1C2  >>= 8;
			R1C1S >>= 8;
			R1C2S >>= 8;
		}
		*reinterpret_cast<uint64*>(Dest    ) = R0;
		*reinterpret_cast<uint64*>(Dest + 8) = R1;
		R0C1  = n0;
		R0C2  = n1;
		R0C1S = n2;
		R0C2S = n3;
		Src  = ADVANCE_POINTER(Src,  Stride);
		Dest = ADVANCE_POINTER(Dest, Stride);
	}

#else
	const int32 blkSize = 16;
	for(int32 v=0; v<blkSize; ++v)
	{
		for(int32 u=0; u<blkSize; ++u)
		{
			int32 sum = (Src[u] + Src[u + 1] + Src[u + Stride] + Src[u + Stride + 1] + Rounding) / 4;
			M4CHECK(sum <= 255);
			Dest[u] = (uint8)sum;
		}
		Src += Stride;
		Dest += Stride;
	}
#endif
}



void M4MemHalfPelInterpolate(void* dst, void* src, int32 stride, int32 x, int32 y, void* mv, uint32 rounding, bool b4x4)
{
	M4CHECK(((size_t)dst & 3) == 0);		// better be the case!

	uint8* cur = (uint8*)dst;
	const uint8* refn = (const uint8*)src;
	const M4_VECTOR* delta = (const M4_VECTOR*)mv;

	int32 ddx, ddy;

	// function entered with actual x/y position
	switch(((delta->x & 1) << 1) + (delta->y & 1))
	{
		case 0:
		{
			// No interpolation, straight copy

			ddx = delta->x / 2;
			ddy = delta->y / 2;

			refn += x + ddx + (y + ddy) * stride;
			cur += x + y * stride;

			if (b4x4)
			{
				_mbCopy16x16(cur, refn, stride);
			}
			else
			{
				_mbCopy8x8(cur, refn, stride);
			}
			break;
		}

		case 1:
		{
			//-------------------------------------------------------
			// Vertical interpolate
			//

			ddx = delta->x / 2;
			ddy = (delta->y - 1) / 2;

			refn += x + ddx + (y + ddy) * stride;
			cur += x + y * stride;

			int32 r = 1 - rounding;

			if (b4x4)
			{
				_mbInterpolateVertical16x16(cur, refn, r, stride);
			}
			else
			{
				_mbInterpolateVertical8x8(cur, refn, r, stride);
			}
			break;
		}

		case 2:
		{
			//-------------------------------------------------------
			// Horizontal interpolate
			//

			ddx = (delta->x-1)/2;
			ddy = delta->y/2;

			refn += x + ddx + (y + ddy) * stride;
			cur += x + y * stride;

			int32 r = 1 - rounding;

			if (b4x4)
			{
				_mbInterpolateHorizontal16x16(cur, refn, r, stride);
			}
			else
			{
				_mbInterpolateHorizontal8x8(cur, refn, r, stride);
			}
			break;
		}

		default:
		{
			//-------------------------------------------------------
			// Both axis interpolate
			//

			ddx = (delta->x - 1) / 2;
			ddy = (delta->y - 1) / 2;

			refn += x + ddx + (y + ddy) * stride;
			cur += x + y * stride;

			int32 r = 2 - rounding;

			if (b4x4)
			{
				_mbInterpolateBoth16x16(cur, refn, r, stride);
			}
			else
			{
				_mbInterpolateBoth8x8(cur, refn, r, stride);
			}
			break;
		}
	}
}





static void _interpolate8x8Simple(uint8* dst, const uint8* src, const int32 x, const int32 y, const int32 stride)
{
	int32 off = x + y * stride;
	src += off;
	dst += off;
	_mbBlendSrcDst8x8(dst, src, stride);
}

static void _interpolate16x16Simple(uint8* dst, const uint8* src, const int32 x, const int32 y, const int32 stride)
{
	int32 off = x + y * stride;
	src += off;
	dst += off;
	_mbBlendSrcDst16x16(dst, src, stride);
}


void M4MemOpInterpolateAll(void* _current, int32 mbx, int32 mby, void* _reference)
{
	M4Image* current = (M4Image*)_current;
	const M4Image* reference = (const M4Image*)_reference;
	int32 stridex = current->mImage.texWidth;
	int32 stride2x = stridex / 2;
	int32 pmbx  = mbx << 4;
	int32 pmby  = mby << 4;
	int32 pmbx2 = mbx << 3;
	int32 pmby2 = mby << 3;

	// merge forward and backward images
	uint8* curx = current->mImage.y;
	uint8* refx = reference->mImage.y;
	_interpolate16x16Simple(curx, refx, pmbx, pmby, stridex);
	_interpolate8x8Simple(current->mImage.u, reference->mImage.u, pmbx2, pmby2, stride2x);
	_interpolate8x8Simple(current->mImage.v, reference->mImage.v, pmbx2, pmby2, stride2x);
}

}

