// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"
#include "M4Global.h"
#include "M4MemOps.h"

namespace vdecmpeg4
{

//! We use the following two macros for 'automatic' overloading of the new
//! and delete operations. The effect is a mapping of all memory allocations
//! to the user-established handler
#define  M4_MEMORY_HANDLER			   					\
public:								 					\
	void* operator new(size_t)							\
	{ 													\
		M4CHECK(false && "new not allowed"); 				\
		return (void*)1;								\
	}													\
	void operator delete[](void*)						\
	{ 													\
		M4CHECK(false && "delete[] not allowed");			\
	}													\
	void* operator new[](size_t) 						\
	{ 													\
		M4CHECK(false && "new[] not allowed"); 			\
		return (void*)1;								\
	}



class M4MemHandler
{
public:
	//! Init global decoder instance memory system
	void init(VIDAllocator cbMem, VIDDeallocator cbFree)
	{
		mAllocCB = cbMem;
		mFreeCB = cbFree;
	}

	//! Memory allocation with possible alignment
	void* malloc(size_t size, size_t alignment = 32)
	{
		M4CHECK(mAllocCB);
		void* ptr = (*mAllocCB)((uint32)size, (uint32)alignment);
		return ptr;
	}

	//! Memory release
	void free(void* ptr)
	{
		M4CHECK(mFreeCB);
		if (ptr)
		{
			(*mFreeCB)(ptr);
		}
	}

private:
	VIDAllocator 				mAllocCB;
	VIDDeallocator 				mFreeCB;
};


}

