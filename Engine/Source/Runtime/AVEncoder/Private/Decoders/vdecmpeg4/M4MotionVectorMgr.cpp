// Copyright Epic Games, Inc. All Rights Reserved.
#include "M4MotionVectorMgr.h"

namespace vdecmpeg4
{
#define MV_EQUAL(A,B) ( ((A).x)==((B).x) && ((A).y)==((B).y) )
#ifndef M4MIN
#define M4MIN(X, Y) ((X)<(Y)?(X):(Y))
#endif
#ifndef M4MAX
#define M4MAX(X, Y) ((X)>(Y)?(X):(Y))
#endif

const M4_VECTOR M4MotionVectorMgr::MV_ZERO = { 0, 0 };

// ----------------------------------------------------------------------------
/**
 * placement new
 *
 * @param sz
 * @param memSys
 *
 * @return
 */
void* M4MotionVectorMgr::operator new(size_t sz, M4MemHandler& memSys)
{
	return memSys.malloc(sz);
}


// ----------------------------------------------------------------------------
/**
 * placement delete
 *
 * @param ptr
 */
void M4MotionVectorMgr::operator delete(void* ptr)
{
	((M4MotionVectorMgr*)ptr)->mDecoder->mMemSys.free(ptr);
}


// ----------------------------------------------------------------------------
/**
 * Static creation of class instance
 *
 * @param pDecoder
 * @param memSys
 *
 * @return
 */
M4MotionVectorMgr* M4MotionVectorMgr::create(M4Decoder* pDecoder, M4MemHandler& memSys)
{
	return new(memSys) M4MotionVectorMgr(pDecoder);
}


// ----------------------------------------------------------------------------
/**
 * Destruction of class instance
 *
 * @param pMgr
 * @param memSys
 */
void M4MotionVectorMgr::destroy(M4MotionVectorMgr*& pMgr, M4MemHandler& memSys)
{
	if (pMgr)
	{
		pMgr->~M4MotionVectorMgr();
		memSys.free(pMgr);
		pMgr = nullptr;
	}
}


// ----------------------------------------------------------------------------
/**
 * Constructor
 *
 * @param decoder Current decoder
 */
M4MotionVectorMgr::M4MotionVectorMgr(M4Decoder* decoder)
	: mDecoder(decoder)
{
}


// ----------------------------------------------------------------------------
/**
 * Initialize requird M4MotionVectorMgr members
 */
void M4MotionVectorMgr::init()
{
	// calc motion vector clipping parameters
	int32 scale_fac = 1<<(mDecoder->mBitstreamParser.mFcodeForward - 1);
	mMvClipHigh = (32 * scale_fac) - 1;
	mMvClipLow  = (-32 * scale_fac);
	mMvClipRange = (64 * scale_fac);
}


// ----------------------------------------------------------------------------
/**
 * Just read the motion vector info from the bitstream
 *
 * @param mb
 */
void M4MotionVectorMgr::readMvForward(M4_MB* mb)
{
	uint16 fcode = mDecoder->mBitstreamParser.mFcodeForward;

	mb->mMvResidual = 1;	// set to indicate that forward motion vector was read from bitstream and is valid!

	if (mb->mMode != M4_MBMODE_INTER4V)
	{
		// frame prediction mode
		// read differential motion vector (MVDx, MVDy) from the bitstream
		readMv(mb->mFMv[0], fcode);
	}
	else
	{
		// M4_MBMODE_INTER4V is not possible in field prediction
		M4CHECK((mb->mFlags & M4_MBFLAG_FIELD_PREDICTION_BIT) == 0);
		// read differential motion vector (MVDx, MVDy) from the bitstream
		readMv(mb->mFMv[0], fcode);
		readMv(mb->mFMv[1], fcode);
		readMv(mb->mFMv[2], fcode);
		readMv(mb->mFMv[3], fcode);
	}
}


// ----------------------------------------------------------------------------
/**
 * Do the actual motion vector prediction
 *
 * @param mb
 * @param mbx
 * @param mby
 */
void M4MotionVectorMgr::predictAddForward(M4_MB* mb, int32 mbx, int32 mby)
{
	// Can only forward predict if motion vector was read from bitstream before...
	M4CHECK(mb->mMvResidual != 0);

	// calc the motion vector predictor (Px, Py) based on the previously decoded macroblocks
	M4_VECTOR prediction;
	_predictMv(prediction, mbx, mby, 0);

	if (mb->mMode != M4_MBMODE_INTER4V)
	{
		// frame prediction mode
		// add differential motion vector (MVDx, MVDy) to the motion vector predictor
		addMvForward(mb->mFMv[0], mb->mFMv[0], prediction);

		// copy result to all other motion vectors
        mb->mFMv[3] = mb->mFMv[2] = mb->mFMv[1] = mb->mFMv[0];
		return;
	}

	// M4_MBMODE_INTER4V is not possible in field prediction

	// add differential motion vector (MVDx, MVDy) to the motion vector predictor
	addMvForward(mb->mFMv[0], mb->mFMv[0], prediction);
	_predictMv(prediction, mbx, mby, 1);
	addMvForward(mb->mFMv[1], mb->mFMv[1], prediction);
	_predictMv(prediction, mbx, mby, 2);
	addMvForward(mb->mFMv[2], mb->mFMv[2], prediction);
	_predictMv(prediction, mbx, mby, 3);
	addMvForward(mb->mFMv[3], mb->mFMv[3], prediction);
}


// ----------------------------------------------------------------------------
/**
 * Calc the motion vector predictor (Px, Py) using a Median filter
 *
 * @param res
 * @param mbx
 * @param mby
 * @param block
 */
void M4MotionVectorMgr::_predictMv(M4_VECTOR& res, int32 mbx, int32 mby, uint32 block)
{
	int32 lx, ly, lz;
    int32 tx, ty, tz;
    int32 rx, ry, rz;

	switch (block)
	{
		case 0:
		{
			lx = mbx-1;	ly = mby;		lz = 1;
			tx = mbx;	ty = mby-1;		tz = 2;
			rx = mbx+1;	ry = mby-1;		rz = 2;
			break;
		}
		case 1:
		{
			lx = mbx;	ly = mby;		lz = 0;
			tx = mbx;	ty = mby - 1;	tz = 3;
			rx = mbx+1;	ry = mby - 1;	rz = 2;
			break;
		}
		case 2:
		{
			lx = mbx-1;	ly = mby;		lz = 3;
			tx = mbx;	ty = mby;		tz = 0;
			rx = mbx;	ry = mby;		rz = 1;
			break;
		}
		default:
		{
			lx = mbx;	ly = mby;		lz = 2;
			tx = mbx;	ty = mby;		tz = 0;
			rx = mbx;	ry = mby;		rz = 1;
			break;
		}
	}

	int32 lpos = lx + ly * mDecoder->mMBWidth;
	int32 rpos = rx + ry * mDecoder->mMBWidth;
	int32 tpos = tx + ty * mDecoder->mMBWidth;

    uint32 num = 0;
	uint32 last = 0;
	M4_VECTOR mv[3];

	if (lpos >= 0 && lx >= 0)
	{
		++num;
		last = 0;
		mv[0] = mDecoder->mIPMacroblocks[lpos].mFMv[lz];
	}
	else
	{
		mv[0] = MV_ZERO;
	}

	if (tpos >= 0)
	{
		++num;
		last = 1;
		mv[1] = mDecoder->mIPMacroblocks[tpos].mFMv[tz];
	}
	else
	{
		mv[1] = MV_ZERO;
	}

	if (rpos >= 0 && rx < (int32)mDecoder->mMBWidth)
	{
		++num;
		last = 2;
		mv[2] = mDecoder->mIPMacroblocks[rpos].mFMv[rz];
	}
	else
	{
		mv[2] = MV_ZERO;
	}

	if (num != 1)
	{
		// median filter
		res.x = M4MIN(M4MAX(mv[0].x, mv[1].x), M4MIN(M4MAX(mv[1].x, mv[2].x), M4MAX(mv[0].x, mv[2].x)));
		res.y = M4MIN(M4MAX(mv[0].y, mv[1].y), M4MIN(M4MAX(mv[1].y, mv[2].y), M4MAX(mv[0].y, mv[2].y)));
	}
	else
	{
		res = mv[last];
	}
}

}

