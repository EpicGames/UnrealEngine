// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"
#include "M4Decoder.h"
#include "M4Memory.h"
#include "M4Global.h"
#include "M4MemOps.h"

namespace vdecmpeg4
{

class M4MotionVectorMgr
{
public:
	//! Creation
	static M4MotionVectorMgr* create(M4Decoder* pDecoder, M4MemHandler& memSys);

	//! Destruction
	static void destroy(M4MotionVectorMgr*& pMgr, M4MemHandler& memSys);

	// -----------------------------------------------------------------------------------------

	//!	Init members for current frame
	void init();

	//! Read the motion vector residual from the bitstream
	void readMvForward(M4_MB* mb);

	//! Add the residual to the prediction
	void predictAddForward(M4_MB* mb, int32 mbx, int32 mby);

	//! Add predictor and input motion vector based on forward/backward code
	void addMv(M4_VECTOR& res, uint16 code, const M4_VECTOR& inputMv, const M4_VECTOR& predictor)
	{
		int32 scale_fac = 1 << (code - 1);
		int32 high = (32 * scale_fac) - 1;
		int32 low = (-32 * scale_fac);
		int32 range = (64 * scale_fac);

		res.x = predictor.x + inputMv.x;
		res.y = predictor.y + inputMv.y;

		if (res.x < low)
		{
			res.x += range;
		}
		else if (res.x > high)
		{
			res.x -= range;
		}

		if (res.y < low)
		{
			res.y += range;
		}
		else if (res.y > high)
		{
			res.y -= range;
		}
	}

	//! Add predicted motion vector to FORWARD prediction
	void addMvForward(M4_VECTOR& res, const M4_VECTOR& inputMv, const M4_VECTOR& predictor)
	{
		// Add input motion vector to predictor
		res.x = inputMv.x + predictor.x;
		res.y = inputMv.y + predictor.y;

		// clipping
		if (res.x < mMvClipLow)
		{
			res.x += mMvClipRange;
		}
		else if (res.x > mMvClipHigh)
		{
			res.x -= mMvClipRange;
		}

		if (res.y < mMvClipLow)
		{
			res.y += mMvClipRange;
		}
		else if (res.y > mMvClipHigh)
		{
			res.y -= mMvClipRange;
		}
	}

	//! Read motion vector residual from file
	void readMv(M4_VECTOR& res, uint16 code)
	{
		// Get the differential motion vector (MVDx, MVDy) from the bitstream
		res.x = mDecoder->mBitstreamParser.getMv(code);
		res.y = mDecoder->mBitstreamParser.getMv(code);
	}

	//! Zero motion vector
	static const M4_VECTOR MV_ZERO;

	void* operator new(size_t sz, M4MemHandler& memSys);
	void operator delete(void* ptr);

private:

	//! Default constructor
	M4MotionVectorMgr(M4Decoder* decoder);

	//!	Destructor
	~M4MotionVectorMgr() = default;

	//! motion vector prediction
	void _predictMv(M4_VECTOR& res, int32 mbx, int32 mby, uint32 block);

	//! Default constructor
	M4MotionVectorMgr();

	//! Copy-constructor not implemented
	M4MotionVectorMgr(const M4MotionVectorMgr &pObj);

	//! Assignment operator not implemented
	const M4MotionVectorMgr &operator=(const M4MotionVectorMgr &pObj);

	int32		mMvClipHigh;		//!< motion vector upper border
	int32 		mMvClipLow;			//!< motion vector lower border
	int32 		mMvClipRange;		//!< motion vector range

	//! Reference to our decoder class
	M4Decoder*	mDecoder;

	M4_MEMORY_HANDLER
};


//! GENERIC: Set all motion vectors to 0
inline void M4MotionVectorClear(M4_MB* mb)
{
	FMemory::Memzero(mb->mFMv, sizeof(M4_VECTOR)*4);
	mb->mMvResidual = 0;
}

}

