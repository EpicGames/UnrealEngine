// Copyright Epic Games, Inc. All Rights Reserved.
#include "M4Prediction.h"
#include "M4Global.h"

namespace vdecmpeg4
{

static const int16 defaultACDCValues[15] =
{
	1024,					// default DC predictor
	0, 0, 0, 0, 0, 0, 0,	// default AC row predictors
	0, 0, 0, 0, 0, 0, 0		// default AC column predictors
};


static int32 XDIV_DIV(int32 a, int32 b)
{
	float A = float(a);
	float B = float(b);
	float B2 = float(b >> 1);
	float R;
	if (a > 0)
	{
		R = (A + B2) / B;
	}
	else
	{
		R = (A - B2) / B;
	}
	int32 r = int32(R);
/*
	int32 r;
	if (a > 0)
		r = (a + (b >> 1)) / b;
	else
		r = (a - (b >> 1)) / b;
*/
	return r;
}

static int32 rescaleF32(int32 predictQuant, int32 currentQuant, int32 coeff)
{
	if (coeff)
	{
		return XDIV_DIV(coeff * predictQuant, currentQuant);
	}
	else
	{
		return 0;
	}
}


// ----------------------------------------------------------------------------
/**
 * Perform ACDC prediction
 *
 * @param macroblocks
 *                  macroblock array
 * @param mbx       macroblock x
 * @param mby       macroblock y
 * @param pMBWidth  stride for macroblocks
 * @param block     current yuv subblock (0..5)
 * @param currentQuant
 * @param iDcScaler
 * @param pPredictionOut
 */
void M4PredictionInit(M4_MB* macroblocks, int32 mbx, int32 mby, int32 pMBWidth, uint32 block, uint8 currentQuant, uint16 iDcScaler, int16* pPredictionOut)
{
	int32 index = mbx + mby * pMBWidth;		// current macroblock

	int32 leftQuant = currentQuant;
	int32 topQuant  = currentQuant;

	const int16* pLeft = defaultACDCValues;
	const int16* pTop =  defaultACDCValues;
	const int16* pDiag = defaultACDCValues;

	int16* left = nullptr;
	int16* top = nullptr;
	int16* diag = nullptr;
	int16* current = nullptr;

	// left macroblock
	if (mbx && (macroblocks[index-1].mMode == M4_MBMODE_INTRA || macroblocks[index-1].mMode == M4_MBMODE_INTRA_Q))
	{
		left 	  = macroblocks[index-1].mPredictedValues[0];
		leftQuant = macroblocks[index-1].mQuant;
	}

	// top macroblock
    if (mby && (macroblocks[index-pMBWidth].mMode == M4_MBMODE_INTRA || macroblocks[index-pMBWidth].mMode == M4_MBMODE_INTRA_Q))
	{
		top 	 = macroblocks[index - pMBWidth].mPredictedValues[0];
		topQuant = macroblocks[index - pMBWidth].mQuant;
	}

	// diag macroblock
	if (mbx && mby && (macroblocks[index-1-pMBWidth].mMode == M4_MBMODE_INTRA || macroblocks[index-1-pMBWidth].mMode == M4_MBMODE_INTRA_Q))
	{
		diag = macroblocks[index - 1 - pMBWidth].mPredictedValues[0];
	}

	current = macroblocks[index].mPredictedValues[0];

	// now grab pLeft, pTop, pDiag _blocks_
	// this are the different patterns based on the the dct block 0..5
	switch (block)
	{
		case 0:
		{
			if (left)
			{
				pLeft = left + M4_MBPRED_SIZE;		// pointer to left mb
			}

			if (top)
			{
				pTop = top + (M4_MBPRED_SIZE << 1);
			}

			if (diag)
			{
				pDiag = diag + 3 * M4_MBPRED_SIZE;
			}
			break;
		}

		case 1:
		{
			pLeft = current;				// we use our current MB
			leftQuant = currentQuant;

			if (top)
			{
				pTop  = top + 3 * M4_MBPRED_SIZE;
				pDiag = top + (M4_MBPRED_SIZE << 1);
			}
			break;
		}

		case 2:
		{
			if (left)
			{
				pLeft = left + 3 * M4_MBPRED_SIZE;
				pDiag = left + M4_MBPRED_SIZE;
			}
			pTop = current;
			topQuant = currentQuant;
			break;
		}

		case 3:
		{
			pLeft = current + (M4_MBPRED_SIZE << 1);
			leftQuant = currentQuant;
			pTop = current + M4_MBPRED_SIZE;
			topQuant = currentQuant;
			pDiag = current;
			break;
		}

		case 4:
		{
			if (left)
			{
				pLeft = left + (M4_MBPRED_SIZE << 2);
			}
			if (top)
			{
				pTop = top + (M4_MBPRED_SIZE << 2);
			}
			if (diag)
			{
				pDiag = diag + (M4_MBPRED_SIZE << 2);
			}
			break;
		}

		case 5:
		{
			if (left)
			{
				pLeft = left + 5 * M4_MBPRED_SIZE;
			}
			if (top)
			{
				pTop = top + 5 * M4_MBPRED_SIZE;
			}
			if (diag)
			{
				pDiag = diag + 5 * M4_MBPRED_SIZE;
			}
			break;
		}
	}


	// determine ac prediction direction & ac/dc predictor
	// place rescaled ac/dc predictions into predictors[] for later use
	uint8* acPredDirection = &macroblocks[index].mACPredDirections[block];

	if (M4ABS(pLeft[0] - pDiag[0]) < M4ABS(pDiag[0] - pTop[0]))
	{
		// use vertical prediction, predict from block ABOVE

		// set prediction direction of this block to vertical
		*acPredDirection = 1;

		// set DC value
		pPredictionOut[0] = M4DIVDIV(pTop[0], iDcScaler);

		// set AC vlues
		for(uint32 i=1; i<8; ++i)
		{
			pPredictionOut[i] = (int16)rescaleF32(topQuant, currentQuant, pTop[i]);
//????			pPredictionOut[i] = (int16)rescale(topQuant, currentQuant, pTop[i]);
		}
	}
	else
	{
		// use horizontal prediction, predict from block to LEFT

		// set prediction direction of this block to horizontal
		*acPredDirection = 2;

		// set DC value
		pPredictionOut[0] = M4DIVDIV(pLeft[0], iDcScaler);

		// set AC values
		for(uint32 i=1; i<8; ++i)
		{
			pPredictionOut[i] = (int16)rescaleF32(leftQuant, currentQuant, pLeft[i + 7]);
//????			pPredictionOut[i] = (int16)rescale(leftQuant, currentQuant, pLeft[i + 7]);
		}
	}
}


// ----------------------------------------------------------------------------
/**
 * Add prediction to residual
 *
 * @param mb
 * @param dctBlock
 * @param block
 * @param iDcScaler
 * @param pPrediction
 */
void M4PredictionAdd(M4_MB* mb, int16* dctBlock, uint32 block, uint16 iDcScaler, const int16* pPrediction)
{
    int16* pCurrent = mb->mPredictedValues[block];

	// ALWAYS predict DC value from DC predictor
	dctBlock[0] += pPrediction[0];
	pCurrent[0] = dctBlock[0] * iDcScaler;

	uint8 acPredDirection = mb->mACPredDirections[block];
	if (acPredDirection == 1)
	{
		// vertical prediction
		for(uint32 i=1; i<8; ++i)
		{
			int16 level = dctBlock[i] + pPrediction[i];
			dctBlock[i] = level;
			pCurrent[i] = level;
			pCurrent[i+7] = dctBlock[i*8];
		}
	}
	else if (acPredDirection == 2)
	{
		// horizontal prediction
		for(uint32 i=1; i<8; ++i)
		{
			int16 level = dctBlock[i*8] + pPrediction[i];
			dctBlock[i*8] = level;
			pCurrent[i+7] = level;
			pCurrent[i] = dctBlock[i];
		}
	}
	else
	{
		// no prediction, but save data in macroblock
		// save first ROW and COLUMN in mPredictedValues[block]
		for(uint32 i=1; i<8; ++i)
		{
			pCurrent[i]   = dctBlock[i];
			pCurrent[i+7] = dctBlock[i*8];
		}
	}
}


}

