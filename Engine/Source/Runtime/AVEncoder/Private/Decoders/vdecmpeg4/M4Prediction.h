// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"
#include "M4Global.h"

namespace vdecmpeg4
{

#define M4ABS(X)		(((X)>0)?(X):-(X))
#define M4DIVDIV(A,B)	( (A) > 0 ? ((A)+((B)>>1))/(B) : ((A)-((B)>>1))/(B) )

struct M4_MB;

void M4PredictionInit(M4_MB* macroblocks, int32 mbx, int32 mby, int32 pMBWidth, uint32 block, uint8 currentQuant, uint16 iDcScaler, int16* output);
void M4PredictionAdd(M4_MB* mb, int16* dctBlock, uint32 block, uint16 iDcScaler, const int16* pPrediction);

inline int32 rescale(int32 predictQuant, int32 currentQuant, int32 coeff)
{
	return coeff != 0 ? M4DIVDIV((coeff) * (predictQuant), (currentQuant)) : 0;
}

}

