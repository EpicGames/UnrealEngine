// Copyright Epic Games, Inc. All Rights Reserved.
#include <stdio.h>

#include "vdecmpeg4.h"
#include "M4Decoder.h"
#include "M4Image.h"

namespace vdecmpeg4
{

// ----------------------------------------------------------------------------
/**
 * Create new decoder instance based on setup parameters
 *
 * @param setup
 * @param pNewDecoder
 *
 * @return
 */
VIDError VIDCreateDecoder(const VIDDecoderSetup* setup, VIDDecoder* pNewDecoder)
{
	M4CHECK(pNewDecoder);
	M4CHECK(setup);
	M4CHECK(setup->cbMemAlloc);
	M4CHECK(setup->cbMemFree);

	M4MemHandler tmpHandler;

	M4Decoder* decoder;
	VIDError error = VID_OK;
    *pNewDecoder = nullptr;

	int16 sw = setup->width;
	int16 sh = setup->height;

	// Do some checks about the size of the input image
	if (((sw & 0xf) != 0) || ((sh & 0xf) != 0))
	{
		return VID_ERROR_WIDTH_OR_HEIGHT_NOT_MULTIPLE_OF_16;
	}
	if ((sw>0 && sw<32) || (sh>0 && sh<32))
	{
		return VID_ERROR_WIDTH_OR_HEIGHT_LESS_THAN_32;
	}
	if (sw>1920 || sh>1088)
	{
		return VID_ERROR_INVALID_WIDTH_OR_HEIGHT;
	}

	tmpHandler.init(setup->cbMemAlloc, setup->cbMemFree);

	if ((decoder = M4Decoder::create(tmpHandler)) == nullptr)
	{
		return VID_ERROR_OUT_OF_MEMORY;
	}
	decoder->mCbPrintf = setup->cbReport;

	// Init decoder
	error = decoder->init(setup);
	if (error == VID_OK)
	{
		// If we have some initial size we perform a full buffer alloc here. Otherwise this is delayed
		// until we know this data from the stream
		if (sw != 0 && sh != 0)
		{
			error = decoder->initBuffers(sw, sh);
		}
		if (error == VID_OK)
		{
			*pNewDecoder = decoder;
			return VID_OK;
		}
	}
	M4Decoder::destroy(decoder, tmpHandler);
	return error;
}


// ----------------------------------------------------------------------------
/**
 * Destroy decoder instance
 *
 * @param dec
 */
void VIDDestroyDecoder(VIDDecoder dec)
{
	M4CHECK(dec);
	M4Decoder* decoder = (M4Decoder*)dec;

	M4MemHandler tmpHandler = decoder->mMemSys;
	M4Decoder::destroy(decoder, tmpHandler);
}

// ----------------------------------------------------------------------------
/**
 * Return frame info
 *
 * @param _pImage
 *
 * @return
 */
const VIDImageInfo* VIDGetFrameInfo(const VIDImage* _pImage)
{
	M4CHECK(_pImage);
	M4Image* pImage = (M4Image*)_pImage->_private;
	return &pImage->mImageInfo;
}


// ----------------------------------------------------------------------------
/**
 * Enable bmp output of decoded frames
 *
 * @param dec
 * @param pBaseName
 */
void VIDDebugVideoOutToBMP(VIDDecoder dec, const char *pBaseName)
{
#ifdef _M4_ENABLE_BMP_OUT
	M4CHECK(dec);
	M4Decoder* decoder = (M4Decoder*)dec;
	decoder->SetVideoOutToBMP(pBaseName);
#else
	(void)dec;
	(void)pBaseName;
#endif
}


// ----------------------------------------------------------------------------
/**
 * Attach a stream to the decoder
 *
 * @param decoder
 * @param pStream
 * @param pEvents
 *
 * @return none
 */
VIDError VIDStreamSet(VIDDecoder decoder, VIDStreamIO* pStream, VIDStreamEvents* pEvents)
{
	M4Decoder* pDecoder = (M4Decoder*)decoder;
	M4CHECK(pDecoder);
	return pDecoder->StreamSet(pStream, pEvents);
}


// ----------------------------------------------------------------------------
/**
 * Handle stream processing
 *
 * @param decoder
 * @param time
 * @param result
 *
 * @return none
 */
VIDError VIDStreamDecode(VIDDecoder decoder, float time, const VIDImage** result)
{
	M4Decoder* pDecoder = (M4Decoder*)decoder;
	M4CHECK(pDecoder);
	return pDecoder->StreamDecode(time, result);
}


// ----------------------------------------------------------------------------
/**
 * Set event sink explicitely
 *
 * @param decoder
 * @param pEvents
 *
 * @return none
 */
VIDError VIDStreamEventsSet(VIDDecoder decoder, VIDStreamEvents* pEvents)
{
	M4Decoder* pDecoder = (M4Decoder*)decoder;
	M4CHECK(pDecoder);
	return pDecoder->StreamEventsSet(pEvents);
}


// ----------------------------------------------------------------------------
/**
 * Handle seek notify
 *
 * @param decoder
 *
 * @return none
 */
VIDError VIDStreamSeekNotify(VIDDecoder decoder)
{
	M4Decoder* pDecoder = (M4Decoder*)decoder;
	M4CHECK(pDecoder);
	return pDecoder->StreamSeekNotify();
}

}

