// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Decoders/vdecmpeg4/M4VlcDecoder.h"

namespace vdecmpeg4
{

uint8 M4VlcDecoder::mMaxLevel[4][64] =
{
	{ // intra, last = 0
		27, 10,  5,  4,  3,  3,  3,  3,
		2,  2,  1,  1,  1,  1,  1,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
	},

	{ // intra, last = 1
		8,  3,  2,  2,  2,  2,  2,  1,
		1,  1,  1,  1,  1,  1,  1,  1,
		1,  1,  1,  1,  1,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
	},

	{ // inter, last = 0
		12,  6,  4,  3,  3,  3,  3,  2,
		2,  2,  2,  1,  1,  1,  1,  1,
		1,  1,  1,  1,  1,  1,  1,  1,
		1,  1,  1,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
	},

	{ // inter, last = 1
		3,  2,  1,  1,  1,  1,  1,  1,
		1,  1,  1,  1,  1,  1,  1,  1,
		1,  1,  1,  1,  1,  1,  1,  1,
		1,  1,  1,  1,  1,  1,  1,  1,
		1,  1,  1,  1,  1,  1,  1,  1,
		1,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
	}
};

uint8 M4VlcDecoder::mMaxRun[4][256] =
{
	{ // intra, last = 0
		0, 14,  9,  7,  3,  2,  1,  1,
		1,  1,  1,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
	},

	{ // intra, last = 1
		0, 20,  6,  1,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
	},

	{ // inter, last = 0
		0, 26, 10,  6,  2,  1,  1,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
	},

	{ // inter, last = 1
		0, 40,  1,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
		0,  0,  0,  0,  0,  0,  0,  0,
	}
};

const M4_VLC M4VlcDecoder::mTabCbpCIntra[64] =
{
	VLC_ERROR,
	MK_VLC(20,6), MK_VLC(36,6), MK_VLC(52,6), MK_VLC(4,4), MK_VLC(4,4), MK_VLC(4,4),
	MK_VLC(4,4), MK_VLC(19,3), MK_VLC(19,3), MK_VLC(19,3), MK_VLC(19,3), MK_VLC(19,3),
	MK_VLC(19,3), MK_VLC(19,3), MK_VLC(19,3), MK_VLC(35,3), MK_VLC(35,3), MK_VLC(35,3),
	MK_VLC(35,3), MK_VLC(35,3), MK_VLC(35,3), MK_VLC(35,3), MK_VLC(35,3), MK_VLC(51,3),
	MK_VLC(51,3), MK_VLC(51,3), MK_VLC(51,3), MK_VLC(51,3), MK_VLC(51,3), MK_VLC(51,3),
	MK_VLC(51,3), MK_VLC(3, 1),
	MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1),
	MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1),
	MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1),
	MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1),
	MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1), MK_VLC(3, 1),
	MK_VLC(3, 1)
};


const M4_VLC M4VlcDecoder::mTabCbpCInter[257] =
{
	VLC_ERROR,
	MK_VLC(255,9), MK_VLC(52,9), MK_VLC(36,9), MK_VLC(20,9), MK_VLC(49,9), MK_VLC(35,8), MK_VLC(35,8), MK_VLC(19,8), MK_VLC(19,8),
	MK_VLC(50,8), MK_VLC(50,8), MK_VLC(51,7), MK_VLC(51,7), MK_VLC(51,7), MK_VLC(51,7), MK_VLC(34,7), MK_VLC(34,7), MK_VLC(34,7),
	MK_VLC(34,7), MK_VLC(18,7), MK_VLC(18,7), MK_VLC(18,7), MK_VLC(18,7), MK_VLC(33,7), MK_VLC(33,7), MK_VLC(33,7), MK_VLC(33,7),
	MK_VLC(17,7), MK_VLC(17,7), MK_VLC(17,7), MK_VLC(17,7), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6),
	MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(48,6), MK_VLC(48,6), MK_VLC(48,6), MK_VLC(48,6), MK_VLC(48,6), MK_VLC(48,6),
	MK_VLC(48,6), MK_VLC(48,6), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5),
	MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5), MK_VLC(3,5),
	MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4),
	MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4),
	MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4),
	MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(32,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4),
	MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4),
	MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4),
	MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4), MK_VLC(16,4),
	MK_VLC(16,4), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3),
	MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3),
	MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3),
	MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3),
	MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3),
	MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3),
	MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(2,3),
	MK_VLC(2,3), MK_VLC(2,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3),
	MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3),
	MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3),
	MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3),
	MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3),
	MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3),
	MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3),
	MK_VLC(1,3), MK_VLC(1,3), MK_VLC(1,3), MK_VLC(0,1)
};

const M4_VLC M4VlcDecoder::mTabCbpY[64] =
{
	VLC_ERROR, VLC_ERROR, MK_VLC(6,6),  MK_VLC(9,6),  MK_VLC(8,5),  MK_VLC(8,5),  MK_VLC(4,5),  MK_VLC(4,5),
	MK_VLC(2,5),  MK_VLC(2,5),  MK_VLC(1,5),  MK_VLC(1,5),  MK_VLC(0,4),  MK_VLC(0,4),  MK_VLC(0,4),  MK_VLC(0,4),
	MK_VLC(12,4), MK_VLC(12,4), MK_VLC(12,4), MK_VLC(12,4), MK_VLC(10,4), MK_VLC(10,4), MK_VLC(10,4), MK_VLC(10,4),
	MK_VLC(14,4), MK_VLC(14,4), MK_VLC(14,4), MK_VLC(14,4), MK_VLC(5,4),  MK_VLC(5,4),  MK_VLC(5,4),  MK_VLC(5,4),
	MK_VLC(13,4), MK_VLC(13,4), MK_VLC(13,4), MK_VLC(13,4), MK_VLC(3,4),  MK_VLC(3,4),  MK_VLC(3,4),  MK_VLC(3,4),
	MK_VLC(11,4), MK_VLC(11,4), MK_VLC(11,4), MK_VLC(11,4), MK_VLC(7,4),  MK_VLC(7,4),  MK_VLC(7,4),  MK_VLC(7,4),
	MK_VLC(15, 2), MK_VLC(15, 2), MK_VLC(15, 2), MK_VLC(15, 2), MK_VLC(15, 2), MK_VLC(15, 2), MK_VLC(15, 2),
	MK_VLC(15, 2), MK_VLC(15, 2), MK_VLC(15, 2), MK_VLC(15, 2), MK_VLC(15, 2), MK_VLC(15, 2), MK_VLC(15, 2),
	MK_VLC(15, 2), MK_VLC(15, 2)
};

M4_VLC M4VlcDecoder::mTabTMNMV0[] =
{
	MK_VLC(3,4), MK_VLC(-3,4), MK_VLC(2,3), MK_VLC(2,3), MK_VLC(-2,3), MK_VLC(-2,3), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2),
	MK_VLC(-1,2), MK_VLC(-1,2), MK_VLC(-1,2), MK_VLC(-1,2)
};

M4_VLC M4VlcDecoder::mTabTMNMV1[] =
{
	MK_VLC(12,10), MK_VLC(-12,10), MK_VLC(11,10), MK_VLC(-11,10), MK_VLC(10,9), MK_VLC(10,9), MK_VLC(-10,9), MK_VLC(-10,9),
	MK_VLC(9,9), MK_VLC(9,9), MK_VLC(-9,9), MK_VLC(-9,9), MK_VLC(8,9), MK_VLC(8,9), MK_VLC(-8,9), MK_VLC(-8,9), MK_VLC(7,7), MK_VLC(7,7),
	MK_VLC(7,7), MK_VLC(7,7), MK_VLC(7,7), MK_VLC(7,7), MK_VLC(7,7), MK_VLC(7,7), MK_VLC(-7,7), MK_VLC(-7,7), MK_VLC(-7,7), MK_VLC(-7,7),
	MK_VLC(-7,7), MK_VLC(-7,7), MK_VLC(-7,7), MK_VLC(-7,7), MK_VLC(6,7), MK_VLC(6,7), MK_VLC(6,7), MK_VLC(6,7), MK_VLC(6,7), MK_VLC(6,7),
	MK_VLC(6,7), MK_VLC(6,7), MK_VLC(-6,7), MK_VLC(-6,7), MK_VLC(-6,7), MK_VLC(-6,7), MK_VLC(-6,7), MK_VLC(-6,7), MK_VLC(-6,7),
	MK_VLC(-6,7), MK_VLC(5,7), MK_VLC(5,7), MK_VLC(5,7), MK_VLC(5,7), MK_VLC(5,7), MK_VLC(5,7), MK_VLC(5,7), MK_VLC(5,7), MK_VLC(-5,7),
	MK_VLC(-5,7), MK_VLC(-5,7), MK_VLC(-5,7), MK_VLC(-5,7), MK_VLC(-5,7), MK_VLC(-5,7), MK_VLC(-5,7), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6),
	MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6), MK_VLC(4,6),
	MK_VLC(4,6), MK_VLC(4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6),
	MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6), MK_VLC(-4,6)
};

M4_VLC M4VlcDecoder::mTabTMNMV2[] =
{
	MK_VLC(32,12), MK_VLC(-32,12), MK_VLC(31,12), MK_VLC(-31,12), MK_VLC(30,11), MK_VLC(30,11), MK_VLC(-30,11), MK_VLC(-30,11),
	MK_VLC(29,11), MK_VLC(29,11), MK_VLC(-29,11), MK_VLC(-29,11), MK_VLC(28,11), MK_VLC(28,11), MK_VLC(-28,11), MK_VLC(-28,11),
	MK_VLC(27,11), MK_VLC(27,11), MK_VLC(-27,11), MK_VLC(-27,11), MK_VLC(26,11), MK_VLC(26,11), MK_VLC(-26,11), MK_VLC(-26,11),
	MK_VLC(25,11), MK_VLC(25,11), MK_VLC(-25,11), MK_VLC(-25,11), MK_VLC(24,10), MK_VLC(24,10), MK_VLC(24,10), MK_VLC(24,10),
	MK_VLC(-24,10), MK_VLC(-24,10), MK_VLC(-24,10), MK_VLC(-24,10), MK_VLC(23,10), MK_VLC(23,10), MK_VLC(23,10), MK_VLC(23,10),
	MK_VLC(-23,10), MK_VLC(-23,10), MK_VLC(-23,10), MK_VLC(-23,10), MK_VLC(22,10), MK_VLC(22,10), MK_VLC(22,10), MK_VLC(22,10),
	MK_VLC(-22,10), MK_VLC(-22,10), MK_VLC(-22,10), MK_VLC(-22,10), MK_VLC(21,10), MK_VLC(21,10), MK_VLC(21,10), MK_VLC(21,10),
	MK_VLC(-21,10), MK_VLC(-21,10), MK_VLC(-21,10), MK_VLC(-21,10), MK_VLC(20,10), MK_VLC(20,10), MK_VLC(20,10), MK_VLC(20,10),
	MK_VLC(-20,10), MK_VLC(-20,10), MK_VLC(-20,10), MK_VLC(-20,10), MK_VLC(19,10), MK_VLC(19,10), MK_VLC(19,10), MK_VLC(19,10),
	MK_VLC(-19,10), MK_VLC(-19,10), MK_VLC(-19,10), MK_VLC(-19,10), MK_VLC(18,10), MK_VLC(18,10), MK_VLC(18,10), MK_VLC(18,10),
	MK_VLC(-18,10), MK_VLC(-18,10), MK_VLC(-18,10), MK_VLC(-18,10), MK_VLC(17,10), MK_VLC(17,10), MK_VLC(17,10), MK_VLC(17,10),
	MK_VLC(-17,10), MK_VLC(-17,10), MK_VLC(-17,10), MK_VLC(-17,10), MK_VLC(16,10), MK_VLC(16,10), MK_VLC(16,10), MK_VLC(16,10),
	MK_VLC(-16,10), MK_VLC(-16,10), MK_VLC(-16,10), MK_VLC(-16,10), MK_VLC(15,10), MK_VLC(15,10), MK_VLC(15,10), MK_VLC(15,10),
	MK_VLC(-15,10), MK_VLC(-15,10), MK_VLC(-15,10), MK_VLC(-15,10), MK_VLC(14,10), MK_VLC(14,10), MK_VLC(14,10), MK_VLC(14,10),
	MK_VLC(-14,10), MK_VLC(-14,10), MK_VLC(-14,10), MK_VLC(-14,10), MK_VLC(13,10), MK_VLC(13,10), MK_VLC(13,10), MK_VLC(13,10),
	MK_VLC(-13,10), MK_VLC(-13,10), MK_VLC(-13,10), MK_VLC(-13,10)
};

M4_VLC M4VlcDecoder::mTabDCT3D0[] =
{
	MK_VLC(4225,7), MK_VLC(4209,7), MK_VLC(4193,7), MK_VLC(4177,7), MK_VLC(193,7), MK_VLC(177,7),
	MK_VLC(161,7), MK_VLC(4,7), MK_VLC(4161,6), MK_VLC(4161,6), MK_VLC(4145,6), MK_VLC(4145,6),
	MK_VLC(4129,6), MK_VLC(4129,6), MK_VLC(4113,6), MK_VLC(4113,6), MK_VLC(145,6), MK_VLC(145,6),
	MK_VLC(129,6), MK_VLC(129,6), MK_VLC(113,6), MK_VLC(113,6), MK_VLC(97,6), MK_VLC(97,6),
	MK_VLC(18,6), MK_VLC(18,6), MK_VLC(3,6), MK_VLC(3,6), MK_VLC(81,5), MK_VLC(81,5),
	MK_VLC(81,5), MK_VLC(81,5), MK_VLC(65,5), MK_VLC(65,5), MK_VLC(65,5), MK_VLC(65,5),
	MK_VLC(49,5), MK_VLC(49,5), MK_VLC(49,5), MK_VLC(49,5), MK_VLC(4097,4), MK_VLC(4097,4),
	MK_VLC(4097,4), MK_VLC(4097,4), MK_VLC(4097,4), MK_VLC(4097,4), MK_VLC(4097,4), MK_VLC(4097,4),
	MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2),
	MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2),
	MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2),
	MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2),
	MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2), MK_VLC(1,2),
	MK_VLC(1,2), MK_VLC(1,2), MK_VLC(17,3), MK_VLC(17,3), MK_VLC(17,3), MK_VLC(17,3),
	MK_VLC(17,3), MK_VLC(17,3), MK_VLC(17,3), MK_VLC(17,3), MK_VLC(17,3), MK_VLC(17,3),
	MK_VLC(17,3), MK_VLC(17,3), MK_VLC(17,3), MK_VLC(17,3), MK_VLC(17,3), MK_VLC(17,3),
	MK_VLC(33,4), MK_VLC(33,4), MK_VLC(33,4), MK_VLC(33,4), MK_VLC(33,4), MK_VLC(33,4),
	MK_VLC(33,4), MK_VLC(33,4), MK_VLC(2,4), MK_VLC(2,4),MK_VLC(2,4),MK_VLC(2,4),
	MK_VLC(2,4), MK_VLC(2,4),MK_VLC(2,4),MK_VLC(2,4)
};

M4_VLC M4VlcDecoder::mTabDCT3D1[] =
{
	MK_VLC(9,10), MK_VLC(8,10), MK_VLC(4481,9), MK_VLC(4481,9), MK_VLC(4465,9), MK_VLC(4465,9),
	MK_VLC(4449,9), MK_VLC(4449,9), MK_VLC(4433,9), MK_VLC(4433,9), MK_VLC(4417,9), MK_VLC(4417,9),
	MK_VLC(4401,9), MK_VLC(4401,9), MK_VLC(4385,9), MK_VLC(4385,9), MK_VLC(4369,9), MK_VLC(4369,9),
	MK_VLC(4098,9), MK_VLC(4098,9), MK_VLC(353,9), MK_VLC(353,9), MK_VLC(337,9), MK_VLC(337,9),
	MK_VLC(321,9), MK_VLC(321,9), MK_VLC(305,9), MK_VLC(305,9), MK_VLC(289,9), MK_VLC(289,9),
	MK_VLC(273,9), MK_VLC(273,9), MK_VLC(257,9), MK_VLC(257,9), MK_VLC(241,9), MK_VLC(241,9),
	MK_VLC(66,9), MK_VLC(66,9), MK_VLC(50,9), MK_VLC(50,9), MK_VLC(7,9), MK_VLC(7,9),
	MK_VLC(6,9), MK_VLC(6,9), MK_VLC(4353,8), MK_VLC(4353,8), MK_VLC(4353,8), MK_VLC(4353,8),
	MK_VLC(4337,8), MK_VLC(4337,8), MK_VLC(4337,8), MK_VLC(4337,8), MK_VLC(4321,8), MK_VLC(4321,8),
	MK_VLC(4321,8), MK_VLC(4321,8), MK_VLC(4305,8), MK_VLC(4305,8), MK_VLC(4305,8), MK_VLC(4305,8),
	MK_VLC(4289,8), MK_VLC(4289,8), MK_VLC(4289,8), MK_VLC(4289,8), MK_VLC(4273,8), MK_VLC(4273,8),
	MK_VLC(4273,8), MK_VLC(4273,8), MK_VLC(4257,8), MK_VLC(4257,8), MK_VLC(4257,8), MK_VLC(4257,8),
	MK_VLC(4241,8), MK_VLC(4241,8), MK_VLC(4241,8), MK_VLC(4241,8), MK_VLC(225,8), MK_VLC(225,8),
	MK_VLC(225,8), MK_VLC(225,8), MK_VLC(209,8), MK_VLC(209,8), MK_VLC(209,8), MK_VLC(209,8),
	MK_VLC(34,8), MK_VLC(34,8), MK_VLC(34,8), MK_VLC(34,8), MK_VLC(19,8), MK_VLC(19,8),
	MK_VLC(19,8), MK_VLC(19,8), MK_VLC(5,8), MK_VLC(5,8), MK_VLC(5,8), MK_VLC(5,8)
};

M4_VLC M4VlcDecoder::mTabDCT3D2[] =
{
	MK_VLC(4114,11), MK_VLC(4114,11), MK_VLC(4099,11), MK_VLC(4099,11), MK_VLC(11,11), MK_VLC(11,11),
	MK_VLC(10,11), MK_VLC(10,11), MK_VLC(4545,10), MK_VLC(4545,10), MK_VLC(4545,10), MK_VLC(4545,10),
	MK_VLC(4529,10), MK_VLC(4529,10), MK_VLC(4529,10), MK_VLC(4529,10), MK_VLC(4513,10), MK_VLC(4513,10),
	MK_VLC(4513,10), MK_VLC(4513,10), MK_VLC(4497,10), MK_VLC(4497,10), MK_VLC(4497,10), MK_VLC(4497,10),
	MK_VLC(146,10), MK_VLC(146,10), MK_VLC(146,10), MK_VLC(146,10), MK_VLC(130,10), MK_VLC(130,10),
	MK_VLC(130,10), MK_VLC(130,10), MK_VLC(114,10), MK_VLC(114,10), MK_VLC(114,10), MK_VLC(114,10),
	MK_VLC(98,10), MK_VLC(98,10), MK_VLC(98,10), MK_VLC(98,10), MK_VLC(82,10), MK_VLC(82,10),
	MK_VLC(82,10), MK_VLC(82,10), MK_VLC(51,10), MK_VLC(51,10), MK_VLC(51,10), MK_VLC(51,10),
	MK_VLC(35,10), MK_VLC(35,10), MK_VLC(35,10), MK_VLC(35,10), MK_VLC(20,10), MK_VLC(20,10),
	MK_VLC(20,10), MK_VLC(20,10), MK_VLC(12,11), MK_VLC(12,11), MK_VLC(21,11), MK_VLC(21,11),
	MK_VLC(369,11), MK_VLC(369,11), MK_VLC(385,11), MK_VLC(385,11), MK_VLC(4561,11), MK_VLC(4561,11),
	MK_VLC(4577,11), MK_VLC(4577,11), MK_VLC(4593,11), MK_VLC(4593,11), MK_VLC(4609,11), MK_VLC(4609,11),
	MK_VLC(22,12), MK_VLC(36,12), MK_VLC(67,12), MK_VLC(83,12), MK_VLC(99,12), MK_VLC(162,12),
	MK_VLC(401,12), MK_VLC(417,12), MK_VLC(4625,12), MK_VLC(4641,12), MK_VLC(4657,12), MK_VLC(4673,12),
	MK_VLC(4689,12), MK_VLC(4705,12), MK_VLC(4721,12), MK_VLC(4737,12), MK_VLC(7167,7),
	MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7),
	MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7),
	MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7),
	MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7),
	MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7), MK_VLC(7167,7),
	MK_VLC(7167,7)
};


// New tables for Intra luminance blocks
// codes as code, len

// Intra Table, i >= 512
M4_VLC M4VlcDecoder::mTabDCT3D3[] =
{
	MK_VLC(0x10401, 7), MK_VLC(0x10301, 7), MK_VLC(0x00601, 7), MK_VLC(0x10501, 7),
	MK_VLC(0x00701, 7), MK_VLC(0x00202, 7), MK_VLC(0x00103, 7), MK_VLC(0x00009, 7),
	MK_VLC(0x10002, 6), MK_VLC(0x10002, 6), MK_VLC(0x00501, 6), MK_VLC(0x00501, 6),
	MK_VLC(0x10201, 6), MK_VLC(0x10201, 6), MK_VLC(0x10101, 6), MK_VLC(0x10101, 6),
	MK_VLC(0x00401, 6), MK_VLC(0x00401, 6), MK_VLC(0x00301, 6), MK_VLC(0x00301, 6),
	MK_VLC(0x00008, 6), MK_VLC(0x00008, 6), MK_VLC(0x00007, 6), MK_VLC(0x00007, 6),
	MK_VLC(0x00102, 6), MK_VLC(0x00102, 6), MK_VLC(0x00006, 6), MK_VLC(0x00006, 6),
	MK_VLC(0x00201, 5), MK_VLC(0x00201, 5), MK_VLC(0x00201, 5), MK_VLC(0x00201, 5),
	MK_VLC(0x00005, 5), MK_VLC(0x00005, 5), MK_VLC(0x00005, 5), MK_VLC(0x00005, 5),
	MK_VLC(0x00004, 5), MK_VLC(0x00004, 5), MK_VLC(0x00004, 5), MK_VLC(0x00004, 5),
	MK_VLC(0x10001, 4), MK_VLC(0x10001, 4), MK_VLC(0x10001, 4), MK_VLC(0x10001, 4),
	MK_VLC(0x10001, 4), MK_VLC(0x10001, 4), MK_VLC(0x10001, 4), MK_VLC(0x10001, 4),
	MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2),
	MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2),
	MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2),
	MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2),
	MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2),
	MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2),
	MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2),
	MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2), MK_VLC(0x00001, 2),
	MK_VLC(0x00002, 3), MK_VLC(0x00002, 3), MK_VLC(0x00002, 3), MK_VLC(0x00002, 3),
	MK_VLC(0x00002, 3), MK_VLC(0x00002, 3), MK_VLC(0x00002, 3), MK_VLC(0x00002, 3),
	MK_VLC(0x00002, 3), MK_VLC(0x00002, 3), MK_VLC(0x00002, 3), MK_VLC(0x00002, 3),
	MK_VLC(0x00002, 3), MK_VLC(0x00002, 3), MK_VLC(0x00002, 3), MK_VLC(0x00002, 3),
	MK_VLC(0x00101, 4), MK_VLC(0x00101, 4), MK_VLC(0x00101, 4), MK_VLC(0x00101, 4),
	MK_VLC(0x00101, 4), MK_VLC(0x00101, 4), MK_VLC(0x00101, 4), MK_VLC(0x00101, 4),
	MK_VLC(0x00003, 4), MK_VLC(0x00003, 4), MK_VLC(0x00003, 4), MK_VLC(0x00003, 4),
	MK_VLC(0x00003, 4), MK_VLC(0x00003, 4), MK_VLC(0x00003, 4), MK_VLC(0x00003, 4)
};

// Intra Table, i >= 128
M4_VLC M4VlcDecoder::mTabDCT3D4[] =
{
	MK_VLC(0x00012,10), MK_VLC(0x00011,10), MK_VLC(0x10e01, 9), MK_VLC(0x10e01, 9),
	MK_VLC(0x10d01, 9), MK_VLC(0x10d01, 9), MK_VLC(0x10c01, 9), MK_VLC(0x10c01, 9),
	MK_VLC(0x10b01, 9), MK_VLC(0x10b01, 9), MK_VLC(0x10a01, 9), MK_VLC(0x10a01, 9),
	MK_VLC(0x10102, 9), MK_VLC(0x10102, 9), MK_VLC(0x10004, 9), MK_VLC(0x10004, 9),
	MK_VLC(0x00c01, 9), MK_VLC(0x00c01, 9), MK_VLC(0x00b01, 9), MK_VLC(0x00b01, 9),
	MK_VLC(0x00702, 9), MK_VLC(0x00702, 9), MK_VLC(0x00602, 9), MK_VLC(0x00602, 9),
	MK_VLC(0x00502, 9), MK_VLC(0x00502, 9), MK_VLC(0x00303, 9), MK_VLC(0x00303, 9),
	MK_VLC(0x00203, 9), MK_VLC(0x00203, 9), MK_VLC(0x00106, 9), MK_VLC(0x00106, 9),
	MK_VLC(0x00105, 9), MK_VLC(0x00105, 9), MK_VLC(0x00010, 9), MK_VLC(0x00010, 9),
	MK_VLC(0x00402, 9), MK_VLC(0x00402, 9), MK_VLC(0x0000f, 9), MK_VLC(0x0000f, 9),
	MK_VLC(0x0000e, 9), MK_VLC(0x0000e, 9), MK_VLC(0x0000d, 9), MK_VLC(0x0000d, 9),
	MK_VLC(0x10801, 8), MK_VLC(0x10801, 8), MK_VLC(0x10801, 8), MK_VLC(0x10801, 8),
	MK_VLC(0x10701, 8), MK_VLC(0x10701, 8), MK_VLC(0x10701, 8), MK_VLC(0x10701, 8),
	MK_VLC(0x10601, 8), MK_VLC(0x10601, 8), MK_VLC(0x10601, 8), MK_VLC(0x10601, 8),
	MK_VLC(0x10003, 8), MK_VLC(0x10003, 8), MK_VLC(0x10003, 8), MK_VLC(0x10003, 8),
	MK_VLC(0x00a01, 8), MK_VLC(0x00a01, 8), MK_VLC(0x00a01, 8), MK_VLC(0x00a01, 8),
	MK_VLC(0x00901, 8), MK_VLC(0x00901, 8), MK_VLC(0x00901, 8), MK_VLC(0x00901, 8),
	MK_VLC(0x00801, 8), MK_VLC(0x00801, 8), MK_VLC(0x00801, 8), MK_VLC(0x00801, 8),
	MK_VLC(0x10901, 8), MK_VLC(0x10901, 8), MK_VLC(0x10901, 8), MK_VLC(0x10901, 8),
	MK_VLC(0x00302, 8), MK_VLC(0x00302, 8), MK_VLC(0x00302, 8), MK_VLC(0x00302, 8),
	MK_VLC(0x00104, 8), MK_VLC(0x00104, 8), MK_VLC(0x00104, 8), MK_VLC(0x00104, 8),
	MK_VLC(0x0000c, 8), MK_VLC(0x0000c, 8), MK_VLC(0x0000c, 8), MK_VLC(0x0000c, 8),
	MK_VLC(0x0000b, 8), MK_VLC(0x0000b, 8), MK_VLC(0x0000b, 8), MK_VLC(0x0000b, 8),
	MK_VLC(0x0000a, 8), MK_VLC(0x0000a, 8), MK_VLC(0x0000a, 8), MK_VLC(0x0000a, 8)
};

// Intra Table, i >= 8
M4_VLC M4VlcDecoder::mTabDCT3D5[] =
{
	MK_VLC(0x10007,11), MK_VLC(0x10007,11), MK_VLC(0x10006,11), MK_VLC(0x10006,11),
	MK_VLC(0x00016,11), MK_VLC(0x00016,11), MK_VLC(0x00015,11), MK_VLC(0x00015,11),
	MK_VLC(0x10202,10), MK_VLC(0x10202,10), MK_VLC(0x10202,10), MK_VLC(0x10202,10),
	MK_VLC(0x10103,10), MK_VLC(0x10103,10), MK_VLC(0x10103,10), MK_VLC(0x10103,10),
	MK_VLC(0x10005,10), MK_VLC(0x10005,10), MK_VLC(0x10005,10), MK_VLC(0x10005,10),
	MK_VLC(0x00d01,10), MK_VLC(0x00d01,10), MK_VLC(0x00d01,10), MK_VLC(0x00d01,10),
	MK_VLC(0x00503,10), MK_VLC(0x00503,10), MK_VLC(0x00503,10), MK_VLC(0x00503,10),
	MK_VLC(0x00802,10), MK_VLC(0x00802,10), MK_VLC(0x00802,10), MK_VLC(0x00802,10),
	MK_VLC(0x00403,10), MK_VLC(0x00403,10), MK_VLC(0x00403,10), MK_VLC(0x00403,10),
	MK_VLC(0x00304,10), MK_VLC(0x00304,10), MK_VLC(0x00304,10), MK_VLC(0x00304,10),
	MK_VLC(0x00204,10), MK_VLC(0x00204,10), MK_VLC(0x00204,10), MK_VLC(0x00204,10),
	MK_VLC(0x00107,10), MK_VLC(0x00107,10), MK_VLC(0x00107,10), MK_VLC(0x00107,10),
	MK_VLC(0x00014,10), MK_VLC(0x00014,10), MK_VLC(0x00014,10), MK_VLC(0x00014,10),
	MK_VLC(0x00013,10), MK_VLC(0x00013,10), MK_VLC(0x00013,10), MK_VLC(0x00013,10),
	MK_VLC(0x00017,11), MK_VLC(0x00017,11), MK_VLC(0x00018,11), MK_VLC(0x00018,11),
	MK_VLC(0x00108,11), MK_VLC(0x00108,11), MK_VLC(0x00902,11), MK_VLC(0x00902,11),
	MK_VLC(0x10302,11), MK_VLC(0x10302,11), MK_VLC(0x10402,11), MK_VLC(0x10402,11),
	MK_VLC(0x10f01,11), MK_VLC(0x10f01,11), MK_VLC(0x11001,11), MK_VLC(0x11001,11),
	MK_VLC(0x00019,12), MK_VLC(0x0001a,12), MK_VLC(0x0001b,12), MK_VLC(0x00109,12),
	MK_VLC(0x00603,12), MK_VLC(0x0010a,12), MK_VLC(0x00205,12), MK_VLC(0x00703,12),
	MK_VLC(0x00e01,12), MK_VLC(0x10008,12), MK_VLC(0x10502,12), MK_VLC(0x10602,12),
	MK_VLC(0x11101,12), MK_VLC(0x11201,12), MK_VLC(0x11301,12), MK_VLC(0x11401,12),
	MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7),
	MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7),
	MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7),
	MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7),
	MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7),
	MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7),
	MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7),
	MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7), MK_VLC(0x01bff, 7)
};

const M4_VLC M4VlcDecoder::mDCLumTab[] =
{
	MK_VLC(0, 0),
	MK_VLC(4, 3), MK_VLC(3, 3), MK_VLC(0, 3),
	MK_VLC(2, 2), MK_VLC(2, 2), MK_VLC(1, 2), MK_VLC(1, 2),
};

}

