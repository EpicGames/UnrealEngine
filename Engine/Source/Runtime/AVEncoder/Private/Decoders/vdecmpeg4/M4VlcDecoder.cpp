// Copyright Epic Games, Inc. All Rights Reserved.

#include "M4VlcDecoder.h"
#include "M4Global.h"
#include "M4VlcData.inl"
#include "M4Decoder.h"

namespace vdecmpeg4
{

// ----------------------------------------------------------------------------
/**
 * Initialize VLC lookup tables
 */
void M4VlcDecoder::init()
{
	mIntraCodeTab.mVlcCodeTab[0] = mTabDCT3D3;
	mIntraCodeTab.mVlcCodeTab[1] = mTabDCT3D4;
	mIntraCodeTab.mVlcCodeTab[2] = mTabDCT3D5;

	mInterCodeTab.mVlcCodeTab[0] = mTabDCT3D0;
	mInterCodeTab.mVlcCodeTab[1] = mTabDCT3D1;
	mInterCodeTab.mVlcCodeTab[2] = mTabDCT3D2;

	mInterCodeTab.mMaxLevel = mIntraCodeTab.mMaxLevel = mMaxLevel;
	mInterCodeTab.mMaxRun = mIntraCodeTab.mMaxRun = mMaxRun;
}



void M4InvQuantType1Intra(int16 *output, const int16 *input, uint8 quantiserScale, uint16 DCScaler, const uint8* dequantMtx)
{
    output[0] = input[0] * DCScaler;
	if (output[0] < -2048)
	{
        output[0] = -2048;
	}
    else if (output[0] > 2047)
	{
        output[0] = 2047;
	}

    for(uint32 i=1; i<64; ++i)
	{
		if (input[i] == 0)
		{
            output[i] = 0;
		}
        else if (input[i] < 0)
		{
            int32 level = -input[i];
            level = (level * dequantMtx[i] * quantiserScale) >> 3;
            output[i] = (level <= 2048 ? -(int16)level : -2048);
		}
        else
		{
            int32 level = input[i];
			level = (level * dequantMtx[i] * quantiserScale) >> 3;
			output[i] = (int16)(level <= 2047 ? level : 2047);
		}
	}
}

void M4InvQuantType1Inter(int16 *output, const int16 *input, uint8 quantiserScale, const uint8* dequantMtx)
{
    uint32 sum = 0;
    uint32 i;
    for(i = 0; i < 64; i++)
	{
        if (input[i] == 0)
		{
            output[i] = 0;
		}
        else if (input[i] < 0)
		{
            int32 level = -input[i];
            level = ((2 * level + 1) * dequantMtx[i] * quantiserScale) >> 4;
            output[i] = (int16)(level <= 2048 ? -level : -2048);
		}
        else
		{
            int32 level = input[i];
            level = ((2 * level + 1) * dequantMtx[i] * quantiserScale) >> 4;
            output[i] = (int16)(level <= 2047 ? level : 2047);
		}
M4CHECK(false && "check type cast");
		sum ^= (uint32)output[i];
	}

    // mismatch control
	if ((sum & 1) == 0)
	{
        output[63] ^= 1;
	}
}

}

