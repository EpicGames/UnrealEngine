// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"
#include "M4Memory.h"
#include "M4Bitstream.h"

namespace vdecmpeg4
{
#define MK_VLC(a,b)	\
	( ((a)&0x1ffff) | ((b)<<17) )

#define VLC_CODE(a)	\
	((a)&0x1ffff)

#define VLC_LEN(a)	\
	((a)>>17)

#define VLC_ERROR	MK_VLC(0x1ffff, 0)
#define VLC_ESCAPE 	0x1BFF

typedef uint32 M4_VLC;		//!< Definition of VLC data types


//! helper struct for passing some C++ tables to the ASM routines. Must be initialized only once.
struct M4BlockVlcCodeTab
{
	M4_VLC*		mVlcCodeTab[3];
	void*		mMaxLevel;
	void*		mMaxRun;
};



class M4VlcDecoder
{
public:
	//! Default constructor
	M4VlcDecoder()
	{}

	//!	Destructor
	~M4VlcDecoder() = default;

	//! Init vlc tables
	void init();

	//!
	int32 getCoeffIntraNoAsm(int32& run, int32& last, M4Bitstream& bs)
	{
		uint32 bits = bs.show(12);
		M4CHECK(bits >= 8);

		M4_VLC vlc = getIntraVlcTab(bits);
		M4CHECK(vlc != VLC_ERROR);

		bs.skip(VLC_LEN(vlc));

		int32 level;
		uint32 code = VLC_CODE(vlc);
		if (code != VLC_ESCAPE)
		{
			run = (code >> 8) & 255;
			level = code & 255;
			last = (code >> 16) & 1;
			int32 ret = bs.getBit() ? -level : level;
			return ret;
		}

		uint32 escMode =  bs.show(2);
		if (escMode < 3)
		{
			bs.skip((escMode == 2) ? 2 : 1);

			bits = bs.show(12);
			M4CHECK(bits >= 8);

			vlc = getIntraVlcTab(bits);
			M4CHECK(vlc != VLC_ERROR);

			bs.skip(VLC_LEN(vlc));

			code = VLC_CODE(vlc);
			run = (code >> 8) & 255;
			level = code & 255;
			last = (code >> 16) & 1;

			if (escMode < 2) 										// first escape mode, level is offset
			{
				level += mMaxLevel[last][run]; 						// need to add back the max level
			}
			else if (escMode == 2)							  		// second escape mode, run is offset
			{
				run += mMaxRun[last][level] + 1;
			}
			else
			{
				M4CHECK(0);	//todo....  (needs a final verification)
			}

			int32 ret = bs.getBit() ? -level : level;
			return ret;
		}

		// third escape mode - fixed length codes
		bs.skip(2);
		last = (int32) bs.getBits(1);
		run = (int32) bs.getBits(6);

		bs.skip(1);				// marker
		level = (int32) bs.getBits(12);
		bs.skip(1);				// marker

		return (level & 0x800) ? (level | (-1 ^ 0xfff)) : level;
	}

	//!
	int32 getCoeffInterNoAsm(int32& run, int32& last, M4Bitstream& bs)
	{
		// use ISO Table B-16, max 12 bit of VLC code
		uint32 bits = bs.show(12);
		M4CHECK(bits >= 8);

		M4_VLC vlc = getInterVlcTab(bits);
		M4CHECK(vlc != VLC_ERROR);

		bs.skip(VLC_LEN(vlc));

		int32 level;
		uint32 code = VLC_CODE(vlc);
		if (code != VLC_ESCAPE)
		{
			run = (code >> 4) & 255;
			level = code & 15;
			last = (code >> 12) & 1;
			int32 ret = bs.getBit() ? -level : level;
			return ret;
		}

		uint32 escMode = bs.show(2);
		if (escMode < 3)
		{
			bs.skip((escMode == 2) ? 2 : 1);

			bits = bs.show(12);
			M4CHECK(bits >= 8);

			vlc = getInterVlcTab(bits);
			M4CHECK(vlc != VLC_ERROR);

			bs.skip(VLC_LEN(vlc));

			code = VLC_CODE(vlc);
			run = (code >> 4) & 255;
			level = code & 15;
			last = (code >> 12) & 1;

			if (escMode < 2) 										// first escape mode, level is offset
			{
				level += mMaxLevel[last + 2][run]; 					// need to add back the max level
			}
			else if (escMode == 2)							  		// second escape mode, run is offset
			{
				run += mMaxRun[last + 2][level] + 1;
			}
			else
			{
				M4CHECK(0);	//todo.... (needs a final verification)
			}

			int32 ret = bs.getBit() ? -level : level;
			return ret;
		}

		// third escape mode - fixed length codes
		bs.skip(2);
		last = (int32) bs.getBits(1);

		run = (int32) bs.getBits(6);
		bs.skip(1);				// marker

		level = (int32) bs.getBits(12);
		bs.skip(1);				// marker

		return (level & 0x800) ? (level | (-1 ^ 0xfff)) : level;
	}

private:
	//! Get vlc code for intra block
	M4_VLC getIntraVlcTab(uint32 bits)
	{
		if (bits >=512)
		{
			return mIntraCodeTab.mVlcCodeTab[0][(bits >> 5) - 16];
		}
		else if (bits >= 128)
		{
			return mIntraCodeTab.mVlcCodeTab[1][(bits >> 2) - 32];
		}
		else
		{
			return mIntraCodeTab.mVlcCodeTab[2][bits - 8];
		}
	}

	//! Get vlc code for inter block
	M4_VLC getInterVlcTab(uint32 bits)
	{
		if (bits >=512)
		{
			return mInterCodeTab.mVlcCodeTab[0][(bits >> 5) - 16];
		}
		else if (bits >= 128)
		{
			return mInterCodeTab.mVlcCodeTab[1][(bits >> 2) - 32];
		}
		else
		{
			return mInterCodeTab.mVlcCodeTab[2][bits - 8];
		}
	}

	//! Copy-constructor not implemented
	M4VlcDecoder(const M4VlcDecoder &pObj);

	//! Assignment operator not implemented
	const M4VlcDecoder &operator=(const M4VlcDecoder &pObj);

	M4BlockVlcCodeTab		mIntraCodeTab;
	M4BlockVlcCodeTab		mInterCodeTab;

	static uint8 			mMaxLevel[4][64];
	static uint8 			mMaxRun[4][256];

	static const M4_VLC		mTabCbpCIntra[64];
	static const M4_VLC		mTabCbpCInter[257];
	static const M4_VLC		mTabCbpY[64];

	static M4_VLC 			mTabTMNMV0[];
	static M4_VLC 			mTabTMNMV1[];
	static M4_VLC			mTabTMNMV2[];

	static M4_VLC			mTabDCT3D0[];
	static M4_VLC			mTabDCT3D1[];
	static M4_VLC			mTabDCT3D2[];

	static M4_VLC			mTabDCT3D3[];
	static M4_VLC			mTabDCT3D4[];
	static M4_VLC			mTabDCT3D5[];

	static const M4_VLC 	mDCLumTab[];

	friend class M4BitstreamParser;
};



}

