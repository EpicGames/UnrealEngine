// Copyright Epic Games, Inc. All Rights Reserved.
#include "M4XCmdSingleThread.h"
#include "M4MemOps.h"
#include "M4Image.h"
#include "M4InvQuant.h"
#include "M4idct.h"
#include "M4BitstreamHeaderInfo.h"

namespace vdecmpeg4
{

#define M4RSHIFT(a,b)	((a) > 0 ? ((a) + (1<<((b)-1)))>>(b) : ((a) + (1<<((b)-1))-1)>>(b))
#define M4SIGN(X)		(((X)>0)?1:-1)
#define M4ABS(X)		(((X)>0)?(X):-(X))


// ----------------------------------------------------------------------------
/**
 * Constructor
 */
M4XCmdSingleThread::M4XCmdSingleThread()
{
	mpDctWorkData = nullptr;
	mpDecoder = nullptr;
}

// ----------------------------------------------------------------------------
/**
 * Destructor
 */
M4XCmdSingleThread::~M4XCmdSingleThread()
{
	M4CHECK(mpDctWorkData == nullptr);
}

// ----------------------------------------------------------------------------
/**
 * Handle initialization for command processing
 *
 * @param pDecoder
 *
 * @return true if successful
 */
VIDError M4XCmdSingleThread::Init(M4Decoder* pDecoder)
{
    M4CHECK(mpDctWorkData == nullptr);
	M4CHECK(pDecoder);

	mpDecoder = pDecoder;
	M4MemHandler& memSys = mpDecoder->mMemSys;

	mpDctWorkData = (int16*)memSys.malloc(64*6*sizeof(int16));
	if (!mpDctWorkData)
	{
		mpDecoder = nullptr;
		return VID_ERROR_OUT_OF_MEMORY;
	}
	return VID_OK;
}

// ----------------------------------------------------------------------------
/**
 * Release command processing resources
 */
void M4XCmdSingleThread::Exit()
{
	// Check if Init was called...
	if (!mpDecoder)
	{
		return;
	}

    M4MemHandler& memSys = mpDecoder->mMemSys;
	memSys.free(mpDctWorkData);

	mpDctWorkData = nullptr;
	mpDecoder =  nullptr;
}

// ----------------------------------------------------------------------------
/**
 * Setup required data for movie frame decode start
 *
 * @param pOutput
 * @param pHeaderInfo
 * @param pRefImage
 */
void M4XCmdSingleThread::FrameBegin(M4Image* pOutput, M4BitstreamHeaderInfo* pHeaderInfo, M4Image* pRefImage[2])
{
	mpOutput 	  = pOutput;
	mpHeaderInfo  = pHeaderInfo;
	mpRefImage[0] = pRefImage[0];
	mpRefImage[1] = pRefImage[1];
}

// ----------------------------------------------------------------------------
/**
 * Perform INTRA (I) macroblock update
**/
void M4XCmdSingleThread::XUpdateIntraMB(M4_MB* pMB, int32 mbx, int32 mby, M4BitstreamCacheEntry* pCacheEntry)
{
    int16* pDctFromStream = pCacheEntry->mDctFromBitstream;
	uint16* pDcScaler = pCacheEntry->mDcScaler;

	uint32	dctOffset = 0;
	for(uint32 i=0; i<6; i++, dctOffset += 64)
	{
		if (mpHeaderInfo->mFlags.mQuantType == 0)
		{
			M4InvQuantType0Intra(mpDctWorkData+dctOffset, pDctFromStream+dctOffset, pMB->mQuant, pDcScaler[i]);
		}
		else
		{
			M4InvQuantType1Intra(mpDctWorkData+dctOffset, pDctFromStream+dctOffset, pMB->mQuant, pDcScaler[i], mpHeaderInfo->mInvQuantIntra);
		}
		M4idct(mpDctWorkData+dctOffset);
	}

	pCacheEntry->mState = 0; // free this cache block

	M4MemOpIntraMBAll(mpOutput, mbx, mby, mpDctWorkData);
}

// ----------------------------------------------------------------------------
/**
 * Perform INTER macroblock update
 *
 * @param pMB
 * @param mbx
 * @param mby
 * @param fields
 * @param pCacheEntry
 * @param refImageNo
 */
void M4XCmdSingleThread::XUpdateInterMB(M4_MB* pMB, int32 mbx, int32 mby, M4BitstreamCacheEntry* pCacheEntry, MV_PREDICTION, uint32 refImageNo)
{
	M4CHECK(mpDctWorkData);

	M4_VECTOR d_uv;
	if (pMB->mMode == M4_MBMODE_INTER || pMB->mMode == M4_MBMODE_INTER_Q)
	{
		d_uv = pMB->mFMv[0];
		d_uv.x = (d_uv.x & 3) ? (d_uv.x >> 1) | 1 : d_uv.x / 2;
		d_uv.y = (d_uv.y & 3) ? (d_uv.y >> 1) | 1 : d_uv.y / 2;
	}
	else
	{
		// M4_MBMODE_INTER4V (not possible in interlace mode)
		int32 sum = pMB->mFMv[0].x + pMB->mFMv[1].x + pMB->mFMv[2].x + pMB->mFMv[3].x;
		d_uv.x = (sum == 0 ? 0 : M4SIGN(sum) * ((int32)M4Decoder::mRoundtab[M4ABS(sum) % 16] + (M4ABS(sum) / 16) * 2) );

		sum = pMB->mFMv[0].y + pMB->mFMv[1].y + pMB->mFMv[2].y + pMB->mFMv[3].y;
		d_uv.y = (sum == 0 ? 0 : M4SIGN(sum) * ((int32)M4Decoder::mRoundtab[M4ABS(sum) % 16] + (M4ABS(sum) / 16) * 2) );
	}

	int32 stride = mpOutput->mImage.texWidth;
	uint8* cur = mpOutput->mImage.y;
	uint8* ref = mpRefImage[refImageNo]->mImage.y;

	int32 x = mbx<<4;
	int32 y = mby<<4;

	int32 tx = pMB->mFMv[0].x;
	int32 ty = pMB->mFMv[0].y;
	if (pMB->mFMv[1].x != tx || pMB->mFMv[1].y != ty ||
		pMB->mFMv[2].x != tx || pMB->mFMv[2].y != ty ||
		pMB->mFMv[3].x != tx || pMB->mFMv[3].y != ty)
	{
		M4MemHalfPelInterpolate(cur, ref, stride, x,   y,   &(pMB->mFMv[0]), mpHeaderInfo->mFlags.mRounding);
		M4MemHalfPelInterpolate(cur, ref, stride, x+8, y,   &(pMB->mFMv[1]), mpHeaderInfo->mFlags.mRounding);
		M4MemHalfPelInterpolate(cur, ref, stride, x,   y+8, &(pMB->mFMv[2]), mpHeaderInfo->mFlags.mRounding);
		M4MemHalfPelInterpolate(cur, ref, stride, x+8, y+8, &(pMB->mFMv[3]), mpHeaderInfo->mFlags.mRounding);
	}
	else
	{
		M4MemHalfPelInterpolate(cur, ref, stride, x,   y,   &(pMB->mFMv[0]), mpHeaderInfo->mFlags.mRounding, true);
	}

	x = mbx<<3;
	y = mby<<3;
	int32 stride2 = stride>>1;
	M4MemHalfPelInterpolate(mpOutput->mImage.u, mpRefImage[refImageNo]->mImage.u, stride2, x, y, &d_uv, mpHeaderInfo->mFlags.mRounding);
	M4MemHalfPelInterpolate(mpOutput->mImage.v, mpRefImage[refImageNo]->mImage.v, stride2, x, y, &d_uv, mpHeaderInfo->mFlags.mRounding);

	if (pMB->mCbp)
	{
		M4CHECK(pCacheEntry);
		const int16* pDctFromStream = pCacheEntry->mDctFromBitstream;

		uint32 dctOffset = 0;
		uint32 mask = 1 << 5;
		for(uint32 i = 0; i < 6; i++, dctOffset+=64, mask>>=1)
		{
			// check if any block is coded inside stream
			if (pMB->mCbp & mask)
			{
				if (mpHeaderInfo->mFlags.mQuantType == 0)
				{
					M4InvQuantType0Inter(mpDctWorkData+dctOffset, pDctFromStream+dctOffset, pMB->mQuant);
				}
				else
				{
					M4InvQuantType1Inter(mpDctWorkData+dctOffset, pDctFromStream+dctOffset, pMB->mQuant, mpHeaderInfo->mInvQuantInter);
				}
				M4idct(mpDctWorkData+dctOffset);
			}
		}

		pCacheEntry->mState = 0; // free this cache block
		M4MemOpInterMBAdd(mpOutput, mbx, mby, mpDctWorkData, pMB->mCbp);
	}
}


// ----------------------------------------------------------------------------
/**
 * Decode an INTER macroblock for B frames using interpolation
 *
 * @param mb
 * @param mbx
 * @param mby
 * @param pCacheEntry
 * @param refImageForward
 * @param refImageBackward
 */
void M4XCmdSingleThread::XInterpolateMB(M4_MB* mb, int32 mbx, int32 mby, M4BitstreamCacheEntry* pCacheEntry, uint32 refImageForward, uint32 refImageBackward, uint16)
{
	M4Image* imgForward  = mpRefImage[refImageForward];
	M4Image* imgBackward = mpRefImage[refImageBackward];
	M4Image* pTmpImage   = mpDecoder->mTempImage[0];

	M4_VECTOR f_uv, b_uv;
	if (mb->mMode == M4_MBMODE_INTER || mb->mMode == M4_MBMODE_INTER_Q)
	{
		// only 1 motion vector
		f_uv = mb->mFMv[0];
		f_uv.x = (f_uv.x & 3) ? (f_uv.x >> 1) | 1 : f_uv.x / 2;
		f_uv.y = (f_uv.y & 3) ? (f_uv.y >> 1) | 1 : f_uv.y / 2;

		b_uv = mb->mBMv[0];
		b_uv.x = (b_uv.x & 3) ? (b_uv.x >> 1) | 1 : b_uv.x / 2;
		b_uv.y = (b_uv.y & 3) ? (b_uv.y >> 1) | 1 : b_uv.y / 2;
	}
	else
	{
		// use 4 motion vectors
		int32 sum;
		sum = mb->mFMv[0].x + mb->mFMv[1].x + mb->mFMv[2].x + mb->mFMv[3].x;
		f_uv.x = (sum == 0 ? 0 : M4SIGN(sum) * ((int32)M4Decoder::mRoundtab[M4ABS(sum) % 16] + (M4ABS(sum) / 16) * 2) );

		sum = mb->mFMv[0].y + mb->mFMv[1].y + mb->mFMv[2].y + mb->mFMv[3].y;
		f_uv.y = (sum == 0 ? 0 : M4SIGN(sum) * ((int32)M4Decoder::mRoundtab[M4ABS(sum) % 16] + (M4ABS(sum) / 16) * 2) );

		sum = mb->mBMv[0].x + mb->mBMv[1].x + mb->mBMv[2].x + mb->mBMv[3].x;
		b_uv.x = (sum == 0 ? 0 : M4SIGN(sum) * ((int32)M4Decoder::mRoundtab[M4ABS(sum) % 16] + (M4ABS(sum) / 16) * 2) );

		sum = mb->mBMv[0].y + mb->mBMv[1].y + mb->mBMv[2].y + mb->mBMv[3].y;
		b_uv.y = (sum == 0 ? 0 : M4SIGN(sum) * ((int32)M4Decoder::mRoundtab[M4ABS(sum) % 16] + (M4ABS(sum) / 16) * 2) );

	}

	// perform FORWARD interpolation
	int32 stride = mpOutput->mImage.texWidth;
	uint8* cur = mpOutput->mImage.y;
	uint8* ref = imgForward->mImage.y;
	int32 x = mbx<<4;			// * 16
	int32 y = mby<<4;			// * 16

	int32 tx = mb->mFMv[0].x;
	int32 ty = mb->mFMv[0].y;
	if (mb->mFMv[1].x != tx || mb->mFMv[1].y != ty ||
		mb->mFMv[2].x != tx || mb->mFMv[2].y != ty ||
		mb->mFMv[3].x != tx || mb->mFMv[3].y != ty)
	{
		M4MemHalfPelInterpolate(cur, ref, stride, x,   y,   &(mb->mFMv[0]), mpHeaderInfo->mFlags.mRounding);
		M4MemHalfPelInterpolate(cur, ref, stride, x+8, y,   &(mb->mFMv[1]), mpHeaderInfo->mFlags.mRounding);
		M4MemHalfPelInterpolate(cur, ref, stride, x,   y+8, &(mb->mFMv[2]), mpHeaderInfo->mFlags.mRounding);
		M4MemHalfPelInterpolate(cur, ref, stride, x+8, y+8, &(mb->mFMv[3]), mpHeaderInfo->mFlags.mRounding);
	}
	else
	{
		M4MemHalfPelInterpolate(cur, ref, stride, x,   y,   &(mb->mFMv[0]), mpHeaderInfo->mFlags.mRounding, true);
	}

	int32 x2 = mbx<<3;
	int32 y2 = mby<<3;
	int32 stride2 = stride>>1;	// /2
	M4MemHalfPelInterpolate(mpOutput->mImage.u, imgForward->mImage.u, stride2, x2, y2, &f_uv, mpHeaderInfo->mFlags.mRounding);
	M4MemHalfPelInterpolate(mpOutput->mImage.v, imgForward->mImage.v, stride2, x2, y2, &f_uv, mpHeaderInfo->mFlags.mRounding);

	// perform BACKWARD interpolation (into temp image buffer)
	cur = pTmpImage->mImage.y;
	ref = imgBackward->mImage.y;

	tx = mb->mBMv[0].x;
	ty = mb->mBMv[0].y;
	if (mb->mBMv[1].x != tx || mb->mBMv[1].y != ty ||
		mb->mBMv[2].x != tx || mb->mBMv[2].y != ty ||
		mb->mBMv[3].x != tx || mb->mBMv[3].y != ty)
	{
		M4MemHalfPelInterpolate(cur, ref, stride, x,   y,   &(mb->mBMv[0]), mpHeaderInfo->mFlags.mRounding);
		M4MemHalfPelInterpolate(cur, ref, stride, x+8, y,   &(mb->mBMv[1]), mpHeaderInfo->mFlags.mRounding);
		M4MemHalfPelInterpolate(cur, ref, stride, x,   y+8, &(mb->mBMv[2]), mpHeaderInfo->mFlags.mRounding);
		M4MemHalfPelInterpolate(cur, ref, stride, x+8, y+8, &(mb->mBMv[3]), mpHeaderInfo->mFlags.mRounding);
	}
	else
	{
		M4MemHalfPelInterpolate(cur, ref, stride, x,   y,   &(mb->mBMv[0]), mpHeaderInfo->mFlags.mRounding, true);
	}

	M4MemHalfPelInterpolate(pTmpImage->mImage.u, imgBackward->mImage.u, stride2, x2, y2, &b_uv, mpHeaderInfo->mFlags.mRounding);
	M4MemHalfPelInterpolate(pTmpImage->mImage.v, imgBackward->mImage.v, stride2, x2, y2, &b_uv, mpHeaderInfo->mFlags.mRounding);

	// merge forward and backward images
	M4MemOpInterpolateAll(mpOutput, mbx, mby, pTmpImage);

	if (mb->mCbp)
	{
		M4CHECK(pCacheEntry);
		const int16* pDctFromStream = pCacheEntry->mDctFromBitstream;

		uint32 dctOffset = 0;
		uint32 mask = 1 << 5;
		for(uint32 i=0; i<6; i++, dctOffset += 64, mask>>=1)
		{
			// check if any block is coded inside stream
			if (mb->mCbp & mask)
			{
				if (mpHeaderInfo->mFlags.mQuantType == 0)
				{
					M4InvQuantType0Inter(mpDctWorkData+dctOffset, pDctFromStream+dctOffset, mb->mQuant);
				}
				else
				{
					M4InvQuantType1Inter(mpDctWorkData+dctOffset, pDctFromStream+dctOffset, mb->mQuant, mpHeaderInfo->mInvQuantInter);
				}
				M4idct(mpDctWorkData+dctOffset);
			}
		}

		pCacheEntry->mState = 0; // free this cache block
		M4MemOpInterMBAdd(mpOutput, mbx, mby, mpDctWorkData, mb->mCbp);
	}
}


}

