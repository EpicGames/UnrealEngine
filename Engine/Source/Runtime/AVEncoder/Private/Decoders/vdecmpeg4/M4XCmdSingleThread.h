// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4_Types.h"
#include "vdecmpeg4_ErrorCodes.h"
#include "M4Global.h"
#include "M4MemOps.h"

namespace vdecmpeg4
{

class M4Image;
class M4Decoder;
struct M4BitstreamCacheEntry;
struct M4_MB;
struct M4BitstreamHeaderInfo;
struct VIDImage;
struct VIDDecoderSetup;

void M4ImageCreatePadding(void* image);


class M4XCmdSingleThread
{
public:

	//! Ctor
	M4XCmdSingleThread();
	//! Dtor
	~M4XCmdSingleThread();

	//! Allocate resources
	VIDError Init(M4Decoder* pDecoder);
	//! Release resources
	void Exit();

	// -----------------------------------------------------------------------

	void FrameBegin(M4Image* pOutput, M4BitstreamHeaderInfo* pHeaderInfo, M4Image* pRefImage[2]);
	void FrameEnd()
	{
	}

	// -----------------------------------------------------------------------

	void XCreatePadding(VIDImage* pImage)
	{
		M4ImageCreatePadding(pImage->_private);
	}
	void XCopyMB(/*M4_MB* pMB,*/ int32 mbx, int32 mby)
	{
		M4MemOpInterMBCopyAll(mpOutput, mbx, mby, mpRefImage[0]);
	}
	void XUpdateIntraMB(M4_MB* pMB, int32 mbx, int32 mby, M4BitstreamCacheEntry* pCacheEntry);
	void XUpdateInterMB(M4_MB* pMB, int32 mbx, int32 mby, M4BitstreamCacheEntry* pCacheEntry, MV_PREDICTION mvDir, uint32 refImageNo);
	void XInterpolateMB(M4_MB* pMb, int32 mbx, int32 mby, M4BitstreamCacheEntry* pCacheEntry, uint32 refImageForward, uint32 refImageBackward, uint16 mbLastIdx);

	// -----------------------------------------------------------------------

	int16* GetDctWorkArea() const
	{
		return mpDctWorkData;
	}

	const VIDImageMacroblockInfo& GetMacroblockInfo() const
	{
		return mMacroblockInfo;
	}

protected:

	int16*					mpDctWorkData;
	M4Image* 				mpOutput;
	M4BitstreamHeaderInfo* 	mpHeaderInfo;

	M4Image* 				mpRefImage[2];

	M4Decoder*				mpDecoder;

	VIDImageMacroblockInfo	mMacroblockInfo;
};

}

