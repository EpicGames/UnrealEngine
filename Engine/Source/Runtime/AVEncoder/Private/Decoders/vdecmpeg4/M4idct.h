// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4.h"

namespace vdecmpeg4
{

//! Inverse DCT on 8x8 block
void M4idct(int16* block);

}


