// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4_Build.h"
#include "vdecmpeg4_Platform.h"
#include "vdecmpeg4_ErrorCodes.h"
#include "vdecmpeg4_Image.h"

namespace vdecmpeg4
{

//! Possible decoder initialization flags used in VIDDecoderSetup::flags
enum VID_DECODER_INIT
{
	VID_DECODER_VID_BUFFERS			= (1<<0),			//!< Allocate indicated number of vid buffers - min 3
	VID_DECODER_DEFAULT				= 0
};

//! Possible return values for VIDGetFrameType() function
enum VIDFrameType
{
	VID_FT_ERROR   = -1,			//!< Cannot determine frame type from data
	VID_FT_I_FRAME = 0,				//!< Input data stream is I frame
	VID_FT_P_FRAME,					//!< Input data stream is P frame
	VID_FT_B_FRAME,					//!< Input data stream is B frame
	VID_FT_S_FRAME					//!< Input data stream is S frame
};


class VIDStreamIO;
class VIDStreamEvents;

//! Possible frame types returned via VIDImageInfo
enum VID_IMAGEINFO_FRAMETYPE
{
	VID_IMAGEINFO_FRAMETYPE_UNKNOWN,					//!< Unknown frame
	VID_IMAGEINFO_FRAMETYPE_I,							//!< I frame
	VID_IMAGEINFO_FRAMETYPE_P,							//!< P frame
	VID_IMAGEINFO_FRAMETYPE_B,							//!< B frame
	VID_IMAGEINFO_FRAMETYPE_S,							//!< S frame (gmc)
};

//! Statistical information about the possible mpeg block from which this frame is made.
struct VIDImageMacroblockInfo
{
	uint16					mNumMBinter;			//!< Number of INTER coded macro blocks
	uint16 					mNumMBintra;			//!< Number of INTRA coded macro blocks
	uint16 					mNumMBinterpolated;		//!< Number of interpolated macro blocks
	uint16 					mNumMBnotCoded;			//!< Number of non-coded (direct copy) macro blocks
};

//! Statistical information about decoded frame. This struct is returned via a call to VIDGetFrameInfo
struct VIDImageInfo
{
	VID_IMAGEINFO_FRAMETYPE	mFrameType;					//!< Current frame type
	uint32					mFrameNumber;				//!< Current frame number
	float					mDecodeTimeMs;				//!< Decode time of frame in ms
	uint32					mFrameBytes;				//!< Size of frame input data in bytes
	VIDImageMacroblockInfo	mMacroblockInfo;			//!< See VIDImageMacroblockInfo
};



// ----------------------------------------------------------------------------
/**
 * Create decoder
 *
 * Used to create and initialize the decoder instance. All memory allocations
 * happen inside this method and the amount of required memory is determined
 * by inspecting the values inside the ::VIDDecoderSetup.
 *
 * @param[in]	setup			pointer to ::VIDDecoderSetup. Can be deleted after this method returns.
 * @param[out]	pNewDecoder		pointer to ::VIDDecoder variable which get.
 *
 * @return		::VIDError result
 *
**/
VIDError VIDCreateDecoder(const VIDDecoderSetup* setup, VIDDecoder* pNewDecoder);

// ----------------------------------------------------------------------------
/**
 * Destroy decoder
 *
 * Used to destroy the decoder instance and free all the memory
 *
 * @param[in]	decoder		handle to decoder.
 *
 * @return		none
 *
**/
void VIDDestroyDecoder(VIDDecoder decoder);




// ----------------------------------------------------------------------------
/**
 * Extract image information from frame.
 *
 * Some additional paramters about the decoded frame are stored in the
 * ::VIDImageInfo struct which can be extracted from a decoder returned
 * ::VIDImage result by calling this function.
 *
 * @param[in]	pImage		pointer to decoder created image.
 *
 * @return		pointer to ::VIDImageInfo containing extended results.
 *
**/
const VIDImageInfo* VIDGetFrameInfo(const VIDImage* pImage);

// ----------------------------------------------------------------------------
/**
 * Automatic output of decoded images to disk.
 *
 * If available (see note below), this function activates the automatic output
 * of the decoded images as BMP files to disk. The supplied base name is
 * prepended to the generated file name.
 *
 * @param[in]	decoder			handle to decoder.
 * @param[in]	pBaseName		ptr to filename base to prepend to images.
 *
 * @return		none (even no error if write fails!)
 *
**/
void VIDDebugVideoOutToBMP(VIDDecoder decoder, const char *pBaseName);

// ----------------------------------------------------------------------------
/**
 * Set current stream for playback
 *
 * The stream is 'attached' to the decoder and processing
 * happens in the VIDStreamDecode function.
 *
 * @param[in]	decoder			handle to decoder.
 * @param[in]	pStream			ptr to stream to play.
 *
 * @return		::VIDError result
 *
**/
VIDError VIDStreamSet(VIDDecoder decoder, VIDStreamIO* pStream, VIDStreamEvents* pEvents);

// ----------------------------------------------------------------------------
/**
 * Perform decoding of stream
 *
 * This initiates the processing of all stream.
 * TODO: Check: Callbacks?
 *
 * @param[in]	decoder			handle to decoder.
 * @param[in]	time			TODO: deltaTime or
 *
 * @return		::VIDError result
 *
**/
VIDError VIDStreamDecode(VIDDecoder decoder, float time, const VIDImage** result);


// ----------------------------------------------------------------------------
/**
 * Set stream event interface explicitly
 *
 * @param[in]	decoder			handle to decoder.
 * @param[in]	pEvents			which event sink to use
 *
 * @return		::VIDError result
 *
**/
VIDError VIDStreamEventsSet(VIDDecoder decoder, VIDStreamEvents* pEvents);



// ----------------------------------------------------------------------------
/**
 * Inform decoder about seeking
 *
 * @param[in]	decoder			handle to decoder.
 *
 * @return		::VIDError result
 *
**/
VIDError VIDStreamSeekNotify(VIDDecoder decoder);


}

