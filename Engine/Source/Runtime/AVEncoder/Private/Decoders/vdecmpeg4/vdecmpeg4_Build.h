// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// This project has been optimized by hand and uses several tricks to improve performance.
// Unfortunately this generates a lot of false positives in static code analysis that we disable here for sanity.

//-V::547,523,594

#if defined(_MSC_VER)
#pragma warning(disable : 6385) // the readable size is 'x' bytes, but 'y' bytes may be read
#pragma warning(disable : 6297) // Arithmetic overflow:  32-bit value is shifted, then cast to 64-bit value.  Results might not be an expected value.
#endif
