// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4_Types.h"

namespace vdecmpeg4
{

//! Generic error type
typedef int32 VIDError;


#define _VID_MAKE_ERROR(_rc)    ((VIDError)(0x80000000 | (_rc)))

static constexpr VIDError VID_OK										= 0;								//!< All okay
static constexpr VIDError VID_ERROR_GENERIC								= _VID_MAKE_ERROR(0x1);				//!< Unspecified error
static constexpr VIDError VID_ERROR_OUT_OF_MEMORY						= _VID_MAKE_ERROR(0x2);				//!< Out of memory

static constexpr VIDError VID_ERROR_NOT_VIDEO_STREAM			    	= _VID_MAKE_ERROR(0x10);			//!< Not a video stream
static constexpr VIDError VID_ERROR_BAD_VIDEO_OBJECT			    	= _VID_MAKE_ERROR(0x11);			//!< Unsupported video_object_type

static constexpr VIDError VID_ERROR_WIDTH_OR_HEIGHT_LESS_THAN_32		= _VID_MAKE_ERROR(0x100);			//!< Resolution of input movie with or height is smaller than 32
static constexpr VIDError VID_ERROR_INVALID_WIDTH_OR_HEIGHT				= _VID_MAKE_ERROR(0x101);			//!< Resolution of input movie is invalid
static constexpr VIDError VID_ERROR_WIDTH_OR_HEIGHT_NOT_MULTIPLE_OF_16	= _VID_MAKE_ERROR(0x102);			//!< Resolution of input movie is not multiple of 16
static constexpr VIDError VID_ERROR_INTERLACED_NOT_SUPPORTED			= _VID_MAKE_ERROR(0x103);			//!< Interlace is not supported

static constexpr VIDError VID_ERROR_SETUP_PLATFORM_DATA_INVALID			= _VID_MAKE_ERROR(0x200);			//!< Some generic trouble with platform setup
static constexpr VIDError VID_ERROR_SETUP_NUMBER_OF_VID_BUFFERS_INVALID	= _VID_MAKE_ERROR(0x201);			//!< Number of allocated buffers must be at least 2

static constexpr VIDError VID_ERROR_DECODE_INVALID_VOP					= _VID_MAKE_ERROR(0x1000);			//!< Could not get valid frame type from bitstream during decode.
static constexpr VIDError VID_ERROR_DECODE_STUFFING_NOT_SUPPORTED		= _VID_MAKE_ERROR(0x1001);			//!< Stuffing is not supported.
static constexpr VIDError VID_ERROR_DECODE_GMC_NOT_ENABLED				= _VID_MAKE_ERROR(0x1010);			//!< Found GMC frame in input stream, but GMC frames are not enabled
static constexpr VIDError VID_ERROR_DECODE_NO_VID_BUFFER_AVAILABLE		= _VID_MAKE_ERROR(0x1020);			//!< All vid buffers are busy/blocked

static constexpr VIDError VID_ERROR_MULTITHREADING_INIT					= _VID_MAKE_ERROR(0x9000);			//!< Generic error during mulithreaded init

static constexpr VIDError VID_ERROR_STREAM_NOT_SET						= _VID_MAKE_ERROR(0xF001);			//!< No stream set for processing
static constexpr VIDError VID_ERROR_STREAM_EOF							= _VID_MAKE_ERROR(0xF002);			//!< Stream reported eof
static constexpr VIDError VID_ERROR_STREAM_ERROR						= _VID_MAKE_ERROR(0xF003);			//!< Any form of stream error during read operation
static constexpr VIDError VID_ERROR_STREAM_UNDERFLOW					= _VID_MAKE_ERROR(0xF004);			//!< Not enough data for returning a finished frame. Call decode again.
static constexpr VIDError VID_ERROR_STREAM_VOL_INVALID_SHAPE			= _VID_MAKE_ERROR(0xF010);			//!< video_object_layer_shape is not valid
static constexpr VIDError VID_ERROR_STREAM_VOP_WITHOUT_VOL				= _VID_MAKE_ERROR(0xF020);			//!< Found a VOP without a preceeding VOL
static constexpr VIDError VID_ERROR_STREAM_VOP_NOT_CODED				= _VID_MAKE_ERROR(0xF021);			//!< Found a non-coded vop. This needs to be skipped
static constexpr VIDError VID_ERROR_STREAM_VOP_INVALID_SCALABILITY		= _VID_MAKE_ERROR(0xF022);			//!< scalability == 1 is not supported
static constexpr VIDError VID_ERROR_STREAM_VOP_FRAME_SKIPPED			= _VID_MAKE_ERROR(0xF023);			//!< Found invalid b-fame time code (perhaps seeking?);

#undef _VID_MAKE_ERROR

}

