// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4_Types.h"

namespace vdecmpeg4
{


// ----------------------------------------------------------------------------
/**
 * Texture structure which characterizes an YUV texture.
 *
**/
struct VIDYUVTexture
{
	uint8*		y;						//!< Pointer to beginning of texture for y samples
	uint8*		u;						//!< Pointer to beginning of texture for u samples
	uint8*		v;						//!< Pointer to beginning of texture for v samples
    void*		userData;				//!< Arbitrary data
};


// ----------------------------------------------------------------------------
/**
 * Video image structure.
 *
 * The decoder returns a pointer to a VIDImage for each frame. Please note that
 * the VIDImage is a subrectangle inside the VIDYUVTexture.
**/
struct VIDImage
{
	int16			width;					//!< Width of video image (excluding border region)
	int16			height;					//!< height of video image (excluding border region)

	int16			texWidth;				//!< Total width of video texture containing image (including border region)
	int16			texHeight;	   			//!< Total height of video texture containing image (including border region)

	uint8*			y;						//!< Pointer to beginning of video y samples
	uint8*			u;						//!< Pointer to beginning of video u samples
	uint8* 			v;						//!< Pointer to beginning of video v samples

	void*			_private;

	VIDYUVTexture	texture;				//!< Our parent texture. We are a subrectangle inside this parent.

	double	 		time;					//!< absolute vop time of this frame in seconds

	virtual void Release() const;		//!< Release image when no longer needed and referenced.
};



}

