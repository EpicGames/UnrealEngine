// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4_Types.h"

namespace vdecmpeg4
{

// ----------------------------------------------------------------------------
/**
 * Memory allocation callback.
 *
 * This is used for all internal decoder memory allocations. Allocations are
 * freed via VIDDeallocator.
 *
 * @param	size			Number of bytes to allocate
 * @param	alignment		Alignment to use for this allocation
 * @return					Pointer to memory
*/
typedef void*(*VIDAllocator)   (uint32 size, uint32 alignment);

// ----------------------------------------------------------------------------
/**
 * Message reporting callback
 *
 * @param	pMessage		preformatted message
*/
typedef void (*VIDReporting)	(const char* pMessage);

// ----------------------------------------------------------------------------
/**
 * Memory release/free callback.
 *
 * All allocations via VIDAllocator are freed here.
 *
 * @param	block			Pointer to memory block to free
*/
typedef void (*VIDDeallocator) (void* block);	//!< memory release callback

typedef void* VIDDecoder;						//!< Opaque type of decoder to outside world. See ::VIDCreateDecoder to create a decoder.


// ----------------------------------------------------------------------------
/**
 * Decoder setup information.
 *
 * This struct needs to be setup for proper decoder initialiazion. This happens
 * often after opening the video input, because we need information about
 * the image width and height here.
 *
**/
struct VIDDecoderSetup
{
	uint32				size;			   	 	//!< Total size of this structure (sizeof)
	uint32				flags;					//!< Init flags. See \ref VID_DECODER_INIT
	int16				width;			   	 	//!< Width of expected video image
	int16				height;					//!< Height of expected video image
	uint16				numOfVidBuffers;		//!< Explicetly select number of image buffers via \ref VID_DECODER_VID_BUFFERS
	uint16 				_reserved;
	VIDAllocator		cbMemAlloc;				//!< Regular memory allocation callback
	VIDDeallocator		cbMemFree;				//!< Regular memory dealloction callback
	VIDAllocator		cbMemAllocLockedCache;	//!< Memory allocation callback for Locked Cache memory (can be nullptr)
	VIDReporting		cbReport;				//!< Reporting function callback
};

}

