// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "vdecmpeg4_Types.h"

namespace vdecmpeg4
{

//! Possible resuls for VIDStream methods
enum VIDStreamResult
{
	VID_STREAM_OK = 0,
	VID_STREAM_EOF,
	VID_STREAM_ERROR,
};

// ----------------------------------------------------------------------------
/**
 * Interface for stream communcation between decoder and actual data provider.
**/
class VIDStreamIO
{
public:

	// ----------------------------------------------------------------------------
	/**
	 * Read data from stream
	 *
	 * The decoder calls this method for reading 'requestedDataBytes' from the
	 * stream to the memory pointed via 'pRequestedDataBuffer'. The real amount
	 * of data which can be returned needs to be assigned to 'actualDataBytes'
	 * and a proper VIDStreamResult must be returned.
	 *
	 * @param		pRequestedDataBuffer 		where to put the read data
	 * @param		requestedDataBytes 			actual size request of the decoder
	 * @param		actualDataBytes 			available number of bytes in buffer after read
	 *
	 * @return		VIDStreamResult
	**/
	virtual VIDStreamResult Read(uint8* pRequestedDataBuffer, uint32 requestedDataBytes, uint32& actualDataBytes) = 0;

	// ----------------------------------------------------------------------------
	/**
	 * Check if stream is at eof
     *
	 * @return		True if stream is at eof
	**/
	virtual bool IsEof() = 0;

};


// ----------------------------------------------------------------------------
/**
 * General callback for stream information
**/
class VIDStreamEvents
{
public:

	//! VideoObjectLayer
	struct VOLInfo
	{
		int16 mWidth;
		int16 mHeight;

		int16 mCodedWidth;
		int16 mCodedHeight;

		// Four-bit integer which defines the value of pixel aspect ratio
		// 0x0:  Forbidden
		// 0x1:  1:1 (Square)
		// 0x2:  12:11 (625-type for 4:3 picture)
		// 0x3:  10:11 (525-type for 4:3 picture)
		// 0x4:  16:11 (625-type stretched for 16:9 picture)
		// 0x5:  40:33 (525-type stretched for 16:9 picture)
		// 0xf:  extended PAR - see below
		uint8 mAspectRatio;

		//! Only valid if aspect_ratio_info == 0xf:
		//! This is an 8-bit unsigned integer which indicates the horizontal size of pixel aspect ratio
		uint8	mAspectRatioPARwidth;
		//! Only valid if aspect_ratio_info == 0xf:
		//! This is an 8-bit unsigned integer which indicates the vertical size of pixel aspect ratio
		uint8	mAspectRatioPARheight;

		uint8 mProfileLevel;

		uint16 mFPSNumerator;
		uint16 mFPSDenominator;
	};

	// ----------------------------------------------------------------------------
	/**
	 * Provide information about a new VOL found in the stream
	 *
	 * @param		volInfo 				information about VOL
	**/
	virtual void FoundVideoObjectLayer(const VOLInfo& volInfo) = 0;
};


}


