// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include <CoreMinimal.h>

// Map M4CHECK() to check().
// This is done to easily disable all checking in a debug build at this location here if necessary.
#define M4CHECK check
#define M4CHECKF checkf
