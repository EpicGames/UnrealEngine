// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoEncoderH264_Dummy.h"

#ifdef AVENCODER_VIDEO_ENCODER_AVAILABLE_H264_DUMMY


namespace AVEncoder
{



void FVideoEncoderH264_Dummy::Register(FVideoEncoderFactory& InFactory)
{
	FVideoEncoderInfo	EncoderInfo;
	EncoderInfo.CodecType = ECodecType::H264;
	EncoderInfo.MaxWidth = 1920;
	EncoderInfo.MaxHeight = 1088;
	EncoderInfo.H264.MaxLevel = 31;
	EncoderInfo.H264.SupportedProfiles = AVEncoder::H264Profile_ConstrainedBaseline | AVEncoder::H264Profile_Baseline;

	InFactory.Register(EncoderInfo, []() {
			return TUniquePtr<FVideoEncoder>(new FVideoEncoderH264_Dummy());
		});
}


FVideoEncoderH264_Dummy::FVideoEncoderH264_Dummy()
{
}

FVideoEncoderH264_Dummy::~FVideoEncoderH264_Dummy()
{
}

bool FVideoEncoderH264_Dummy::Setup(TSharedRef<FVideoEncoderInput> InInput, const FInit& InInit)
{
	return false;
}

void FVideoEncoderH264_Dummy::Shutdown()
{
}

void FVideoEncoderH264_Dummy::Encode(const FVideoEncoderInputFrame* InFrame, const FEncodeOptions& InOptions)
{
}


} // namespace AVEncoder


#endif // AVENCODER_VIDEO_ENCODER_AVAILABLE_H264_DUMMY
