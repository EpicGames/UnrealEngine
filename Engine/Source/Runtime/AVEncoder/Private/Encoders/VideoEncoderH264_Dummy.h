// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// #define AVENCODER_VIDEO_ENCODER_AVAILABLE_H264_DUMMY		// enable dummy h264 encoder

#ifdef AVENCODER_VIDEO_ENCODER_AVAILABLE_H264_DUMMY

#include "VideoEncoderFactory.h"


namespace AVEncoder
{

class FVideoEncoderH264_Dummy : public FVideoEncoder
{
public:
	static void Register(FVideoEncoderFactory& InFactory);

	virtual bool Setup(TSharedRef<FVideoEncoderInput> InInput, const FInit& InInit) override;
	virtual void Shutdown() override;
	virtual void Encode(const FVideoEncoderInputFrame* InFrame, const FEncodeOptions& InOptions) override;
private:
	FVideoEncoderH264_Dummy();
	virtual ~FVideoEncoderH264_Dummy();
};

} // namespace AVEncoder

#endif // AVENCODER_VIDEO_ENCODER_AVAILABLE_H264_DUMMY
