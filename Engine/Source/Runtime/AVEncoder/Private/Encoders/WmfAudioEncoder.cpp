// Copyright Epic Games, Inc. All Rights Reserved.

#include "WmfAudioEncoder.h"

#if PLATFORM_WINDOWS

#include "AVEncoder.h"
#include "Logging/LogMacros.h"
#include "MicrosoftCommon.h"
#include "AudioEncoder.h"
#include "IMFSampleWrapper.h"

namespace AVEncoder
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FWmfAudioEncoder : public FAudioEncoder
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
public:

	enum class ECodecType
	{
		AAC
	};

	FWmfAudioEncoder(ECodecType CodecType);
	~FWmfAudioEncoder() override;

	const TCHAR* GetName() const override;
	const TCHAR* GetType() const override;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool Initialize(const FAudioConfig& InConfig) override;
	void Encode(const FAudioFrame& Frame) override;
	FAudioConfig GetConfig() const override;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	void Shutdown() override;

private:
	bool SetInputType();
	bool SetOutputType();
	bool RetrieveStreamInfo();
	bool StartStreaming();

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FIMFSampleWrapper CreateInputSample(const uint8* SampleData, uint32 Size, FTimespan Timestamp, FTimespan Duration);
	FIMFSampleWrapper GetOutputSample();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	ECodecType CodecType;
	FString Name;
	FString Type;

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FAudioConfig Config;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	TRefCountPtr<IMFTransform> Encoder;
	TRefCountPtr<IMFMediaType> OutputType;
	MFT_INPUT_STREAM_INFO InputStreamInfo = {};
	MFT_OUTPUT_STREAM_INFO OutputStreamInfo = {};

	// Keep this as a member variable, to reuse the memory allocation
	// It's used to convert float audio data to PCM
	Audio::TSampleBuffer<int16> PCM16;
};

//////////////////////////////////////////////////////////////////////////
// FWmfAudioEncoder implementation
//////////////////////////////////////////////////////////////////////////
PRAGMA_DISABLE_DEPRECATION_WARNINGS
FWmfAudioEncoder::FWmfAudioEncoder(ECodecType CodecType)
	: CodecType(CodecType)
{
	switch (CodecType)
	{
	case ECodecType::AAC:
		Name = "aac.wmf";
		Type = "aac";
		break;
	default:
		checkNoEntry();
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FWmfAudioEncoder::~FWmfAudioEncoder()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const TCHAR* FWmfAudioEncoder::GetName() const
{
	return *Name;
}

const TCHAR* FWmfAudioEncoder::GetType() const
{
	return *Type;
}


namespace
{
	//
	// Utility function to validate if a given encoder parameter has an acceptable value
	//
	bool ValidateValue(const TCHAR* Name, uint32 Value, const TArray<uint32>& SupportedValues, FString& OutErr)
	{
		for(uint32 V : SupportedValues)
		{
			if (Value == V)
			{
				return true;
			}
		}

		OutErr = FString(Name) + TEXT(" must be (");
		for(uint32 V : SupportedValues)
		{
			OutErr += FString::Printf(
				TEXT("%s%u"),
				OutErr.IsEmpty() ? TEXT("") : TEXT(", "),
				V);
		}
		OutErr += FString::Printf(TEXT("). %u was specified."), Value);
		return false;
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FWmfAudioEncoder::Initialize(const FAudioConfig& InConfig)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{

	//
	// Validate config depending on the codec
	//
	if (CodecType == ECodecType::AAC)
	{
		// See  https://docs.microsoft.com/en-us/windows/desktop/medfound/aac-encoder for details
		FString ErrorStr;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (
			!ValidateValue(TEXT("AAC Bitrate"), InConfig.Bitrate, {12000*8, 16000*8, 20000*8, 24000*8}, ErrorStr) ||
			!ValidateValue(TEXT("AAC Samplerate"), InConfig.Samplerate, { 44100, 48000, 0 }, ErrorStr) ||
			!ValidateValue(TEXT("AAC NumChannels"), InConfig.NumChannels, {1,2,6}, ErrorStr))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			UE_LOGF(LogAVEncoder, Error, "%ls", *ErrorStr);
			return false;
		}
	}
	else
	{
		checkNoEntry();
		return false;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_LOGF(LogAVEncoder, Log, "AudioEncoder config: %d channels, %d Hz, %.2f Kbps", InConfig.NumChannels, InConfig.Samplerate, InConfig.Bitrate / 1000.0f);

	Config = InConfig;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const GUID* CodecGuid=nullptr;
	if (CodecType == ECodecType::AAC)
	{
		CodecGuid = &CLSID_AACMFTEncoder;
	}
	else
	{
		checkNoEntry();
	}

	CHECK_HR(CoCreateInstance(*CodecGuid, nullptr, CLSCTX_INPROC_SERVER, IID_IMFTransform, reinterpret_cast<void**>(&Encoder)));

	//
	// NOTES
	// MP3: The output type must be set before the input. See https://docs.microsoft.com/en-us/windows/win32/medfound/mp3-audio-encoder
	// AAC: Output or input type can be set in either order. Input type first, or output type first.

	if (!SetInputType() || !SetOutputType() || !RetrieveStreamInfo() || !StartStreaming()) 
	{
		Encoder->Release();
		return false;
	}

	return true;
}


bool FWmfAudioEncoder::SetInputType()
{
	TRefCountPtr<IMFMediaType> MediaType;
	CHECK_HR(MFCreateMediaType(MediaType.GetInitReference()));
	CHECK_HR(MediaType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio));
	CHECK_HR(MediaType->SetGUID(MF_MT_SUBTYPE, MFAudioFormat_PCM));
	CHECK_HR(MediaType->SetUINT32(MF_MT_AUDIO_BITS_PER_SAMPLE, 16)); // the only value supported

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CHECK_HR(MediaType->SetUINT32(MF_MT_AUDIO_SAMPLES_PER_SECOND, Config.Samplerate));
	CHECK_HR(MediaType->SetUINT32(MF_MT_AUDIO_NUM_CHANNELS, Config.NumChannels));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	CHECK_HR(Encoder->SetInputType(0, MediaType, 0));

	return true;
}

bool FWmfAudioEncoder::SetOutputType()
{
	CHECK_HR(MFCreateMediaType(OutputType.GetInitReference()));
	CHECK_HR(OutputType->SetGUID(MF_MT_MAJOR_TYPE, MFMediaType_Audio));

	if (CodecType == ECodecType::AAC)
	{
		CHECK_HR(OutputType->SetGUID(MF_MT_SUBTYPE, MFAudioFormat_AAC));
		CHECK_HR(OutputType->SetUINT32(MF_MT_AUDIO_BITS_PER_SAMPLE, 16)); // the only value supported
		
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CHECK_HR(OutputType->SetUINT32(MF_MT_AUDIO_SAMPLES_PER_SECOND, Config.Samplerate));
		CHECK_HR(OutputType->SetUINT32(MF_MT_AUDIO_NUM_CHANNELS, Config.NumChannels));
		CHECK_HR(OutputType->SetUINT32(MF_MT_AUDIO_AVG_BYTES_PER_SECOND, Config.Bitrate / 8));
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	else
	{
		checkNoEntry();
		return false;
	}

	CHECK_HR(Encoder->SetOutputType(0, OutputType, 0));

	return true;
}

bool FWmfAudioEncoder::RetrieveStreamInfo()
{
	CHECK_HR(Encoder->GetInputStreamInfo(0, &InputStreamInfo));
	CHECK_HR(Encoder->GetOutputStreamInfo(0, &OutputStreamInfo));

	return true;
}

bool FWmfAudioEncoder::StartStreaming()
{
	CHECK_HR(Encoder->ProcessMessage(MFT_MESSAGE_NOTIFY_BEGIN_STREAMING, 0));
	CHECK_HR(Encoder->ProcessMessage(MFT_MESSAGE_NOTIFY_START_OF_STREAM, 0));

	return true;
}

void FWmfAudioEncoder::Shutdown()
{
	// Nothing to do
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FWmfAudioEncoder::Encode(const FAudioFrame& Frame)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_LOGF(LogAVEncoder, Verbose, "Audio input: time %.3f, duration %.3f, %d samples", Frame.Timestamp.GetTotalSeconds(), Frame.Duration.GetTotalSeconds(), Frame.Data.GetNumSamples());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	// Convert float audio data to PCM
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	PCM16 = Frame.Data;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FIMFSampleWrapper InputSample = CreateInputSample(
		reinterpret_cast<const uint8*>(PCM16.GetData()),
		PCM16.GetNumSamples()*sizeof(*PCM16.GetData()),
		Frame.Timestamp,
		Frame.Duration);

	if (!InputSample.IsValid())
	{
		return;
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	CHECK_HR_VOID(Encoder->ProcessInput(0, InputSample.GetSample(), 0));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	while (true)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FIMFSampleWrapper OutputSample = GetOutputSample();
		if (!OutputSample.IsValid())
		{
			break;
		}

		DWORD OutputSize;
		CHECK_HR_VOID(OutputSample.GetSample()->GetTotalLength(&OutputSize));

		UE_LOGF(LogAVEncoder, Verbose, "Audio encoded: time %.3f, duration %.3f, size %d", OutputSample.GetTime().GetTotalSeconds(), OutputSample.GetDuration().GetTotalSeconds(), OutputSize);

		FMediaPacket Packet(EPacketType::Audio);
		Packet.Timestamp = OutputSample.GetTime();
		Packet.Duration = OutputSample.GetDuration();
		OutputSample.IterateBuffers([&Packet](int BufferIdx, TArrayView<uint8> Data)
		{
			Packet.Data.Append(Data.GetData(), Data.Num());
			return false; // Terminate, because we only want 1 buffer
		});
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		OnEncodedAudioFrame(Packet);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FIMFSampleWrapper FWmfAudioEncoder::CreateInputSample(const uint8* SampleData, uint32 Size, FTimespan Timestamp, FTimespan Duration)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FIMFSampleWrapper Sample = { EPacketType::Audio };

	if (!Sample.CreateSample())
	{
		return {};
	}

	int32 BufferSize = FMath::Max<int32>(InputStreamInfo.cbSize, Size);
	uint32 Alignment = InputStreamInfo.cbAlignment > 1 ? InputStreamInfo.cbAlignment - 1 : 0;
	TRefCountPtr<IMFMediaBuffer> WmfBuffer;
	CHECK_HR_DEFAULT(MFCreateAlignedMemoryBuffer(BufferSize, Alignment, WmfBuffer.GetInitReference()));

	uint8* Dst = nullptr;
	CHECK_HR_DEFAULT(WmfBuffer->Lock(&Dst, nullptr, nullptr));
	FMemory::Memcpy(Dst, SampleData, Size);
	CHECK_HR_DEFAULT(WmfBuffer->Unlock());

	CHECK_HR_DEFAULT(WmfBuffer->SetCurrentLength(Size));

	CHECK_HR_DEFAULT(Sample.GetSample()->AddBuffer(WmfBuffer));
	Sample.SetTime(Timestamp);
	Sample.SetDuration(Duration);

	return Sample;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FIMFSampleWrapper FWmfAudioEncoder::GetOutputSample()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool bFlag1 = OutputStreamInfo.dwFlags&MFT_OUTPUT_STREAM_PROVIDES_SAMPLES;
	bool bFlag2 = OutputStreamInfo.dwFlags&MFT_OUTPUT_STREAM_CAN_PROVIDE_SAMPLES;

	// Right now, we are always creating our samples, so make sure MFT will be ok with that
	verify((bFlag1 == false && bFlag2 == false) || (bFlag1 == false && bFlag2 == true));

	FIMFSampleWrapper Sample{ EPacketType::Audio };

	//
	// Create output sample
	//
	{
		if (!Sample.CreateSample())
		{
			return {};
		}

		TRefCountPtr<IMFMediaBuffer> Buffer = nullptr;
		uint32 Alignment = OutputStreamInfo.cbAlignment > 1 ? OutputStreamInfo.cbAlignment - 1 : 0;
		CHECK_HR_DEFAULT(MFCreateAlignedMemoryBuffer(OutputStreamInfo.cbSize, Alignment, Buffer.GetInitReference()));
		CHECK_HR_DEFAULT(Sample.GetSample()->AddBuffer(Buffer));
	}


	MFT_OUTPUT_DATA_BUFFER Output = {};
	Output.pSample = Sample.GetSample();

	DWORD Status = 0;
	HRESULT Res = Encoder->ProcessOutput(0, 1, &Output, &Status);
	TRefCountPtr<IMFCollection> Events = Output.pEvents; // unsure this is released
	if (Res == MF_E_TRANSFORM_NEED_MORE_INPUT || !Sample.IsValid())
	{
		return {};
	}
	else if (Res == MF_E_TRANSFORM_STREAM_CHANGE)
	{
		if (!SetOutputType())
		{
			return {};
		}

		return GetOutputSample();
	}
	else if (SUCCEEDED(Res))
	{
		return Sample;
	}
	else
	{
		return {};
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FAudioConfig FWmfAudioEncoder::GetConfig() const
{
	return Config;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

//////////////////////////////////////////////////////////////////////////
// FWmfAudioEncoderFactory implementation
//////////////////////////////////////////////////////////////////////////

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FWmfAudioEncoderFactory::FWmfAudioEncoderFactory()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FWmfAudioEncoderFactory::~FWmfAudioEncoderFactory()
{
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


const TCHAR* FWmfAudioEncoderFactory::GetName() const
{
	return TEXT("wmf");
}

TArray<FString> FWmfAudioEncoderFactory::GetSupportedCodecs() const
{
	TArray<FString> Codecs;
	Codecs.Add(TEXT("aac"));
	return Codecs;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TUniquePtr<FAudioEncoder> FWmfAudioEncoderFactory::CreateEncoder(const FString& Codec)
{
	if (Codec == "aac")
	{
		return TUniquePtr<FAudioEncoder>(new FWmfAudioEncoder(FWmfAudioEncoder::ECodecType::AAC));
	}
	
	else
	{
		UE_LOGF(LogAVEncoder, Error, "FWmfAudioEncoderFactory doesn't support the %ls codec", *Codec);
		return nullptr;
	}
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

}

#endif //PLATFORM_WINDOWS

