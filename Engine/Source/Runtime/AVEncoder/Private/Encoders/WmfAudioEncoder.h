// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"

#if PLATFORM_WINDOWS

#include "AudioEncoderFactory.h"

namespace AVEncoder
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	class FWmfAudioEncoderFactory : public FAudioEncoderFactory
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	public:
		FWmfAudioEncoderFactory();
		~FWmfAudioEncoderFactory() override;
		const TCHAR* GetName() const override;
		TArray<FString> GetSupportedCodecs() const override;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TUniquePtr<FAudioEncoder> CreateEncoder(const FString& Codec) override;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	private:
	};
}

#endif //PLATFORM_WINDOWS

