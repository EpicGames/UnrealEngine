// Copyright Epic Games, Inc. All Rights Reserved.

#include "IMFSampleWrapper.h"
#include "AVEncoder.h"

#if PLATFORM_WINDOWS

#include "MicrosoftCommon.h"

namespace AVEncoder
{

namespace
{
	TRefCountPtr<IMFMediaBuffer> CloneMediaBuffer(const TRefCountPtr<IMFMediaBuffer>& Src)
	{
		uint8* SrcByteBuffer = nullptr;
		DWORD SrcMaxLen = 0;
		DWORD SrcCurrLen = 0;
		verify(SUCCEEDED(Src->Lock(&SrcByteBuffer, &SrcMaxLen, &SrcCurrLen)));

		TRefCountPtr<IMFMediaBuffer> Dest;
		verify(SUCCEEDED(MFCreateMemoryBuffer(SrcCurrLen, Dest.GetInitReference())));

		uint8* DestByteBuffer = nullptr;
		verify(SUCCEEDED(Dest->Lock(&DestByteBuffer, nullptr, nullptr)));
		FMemory::Memcpy(DestByteBuffer, SrcByteBuffer, SrcCurrLen);
		verify(SUCCEEDED(Dest->Unlock()));
		verify(SUCCEEDED(Src->Unlock()));

		verify(SUCCEEDED(Dest->SetCurrentLength(SrcCurrLen)));
		return Dest;
	}
}

//////////////////////////////////////////////////////////////////////////
// FIMFSampleWrapper
//////////////////////////////////////////////////////////////////////////


bool FIMFSampleWrapper::CreateSample()
{
	Sample = nullptr;
	CHECK_HR(MFCreateSample(Sample.GetInitReference()));
	return true;
}

FTimespan FIMFSampleWrapper::GetTime() const
{
	int64 Time;
	HRESULT hr = Sample->GetSampleTime(&Time);
	verifyf(SUCCEEDED(hr), TEXT("%s"), *GetComErrorDescription(hr));
	return Time;
}

void FIMFSampleWrapper::SetTime(FTimespan Time)
{
	verify(SUCCEEDED(Sample->SetSampleTime(Time.GetTicks())));
}

FTimespan FIMFSampleWrapper::GetDuration() const
{
	int64 Duration;
	HRESULT hr = Sample->GetSampleDuration(&Duration);
	verifyf(SUCCEEDED(hr), TEXT("%s"), *GetComErrorDescription(hr));
	return Duration;
}

void FIMFSampleWrapper::SetDuration(FTimespan Duration)
{
	verify(SUCCEEDED(Sample->SetSampleDuration(Duration.GetTicks())));
}

void FIMFSampleWrapper::SetIsKeyFrame()
{
	verify(SUCCEEDED(Sample->SetUINT32(MFSampleExtension_CleanPoint, 1)));
}

bool FIMFSampleWrapper::IsVideoKeyFrame() const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return MediaType == EPacketType::Video && MFGetAttributeUINT32(Sample, MFSampleExtension_CleanPoint, 0) != 0;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

int FIMFSampleWrapper::GetBufferCount() const
{
	DWORD BufferCount = 0;
	verify(SUCCEEDED(Sample->GetBufferCount(&BufferCount)));
	return BufferCount;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FIMFSampleWrapper FIMFSampleWrapper::Clone() const
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FIMFSampleWrapper Res{ MediaType };
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (!Sample)
	{
		return Res;
	}

	verify(Res.CreateSample());

	DWORD Flags = 0;
	verify(SUCCEEDED(Sample->GetSampleFlags(&Flags)));
	verify(SUCCEEDED(Res.GetSample()->SetSampleFlags(Flags)));

	Res.SetTime(GetTime());
	Res.SetDuration(GetDuration());

	if (IsVideoKeyFrame())
	{
		verify(SUCCEEDED(Res.GetSample()->SetUINT32(MFSampleExtension_CleanPoint, 1)));
	}

	DWORD BufferCount = GetBufferCount();

	for (DWORD i = 0; i != BufferCount; ++i)
	{
		TRefCountPtr<IMFMediaBuffer> SrcBuf;
		verify(SUCCEEDED(Sample->GetBufferByIndex(i, SrcBuf.GetInitReference())));

		TRefCountPtr<IMFMediaBuffer> DestBuf = CloneMediaBuffer(SrcBuf);
		verify(SUCCEEDED(Res.GetSample()->AddBuffer(DestBuf)));
	}

	return Res;
}

} // namespace AVEncoder

#endif // PLATFORM_WINDOWS

