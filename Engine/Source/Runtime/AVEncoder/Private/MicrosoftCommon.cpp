// Copyright Epic Games, Inc. All Rights Reserved.

#include "MicrosoftCommon.h"

#if PLATFORM_WINDOWS

#if WMFMEDIA_SUPPORTED_PLATFORM
	#pragma comment(lib, "mfplat")
	#pragma comment(lib, "mfuuid")
	#pragma comment(lib, "Mfreadwrite")
#endif

#endif // PLATFORM_WINDOWS
