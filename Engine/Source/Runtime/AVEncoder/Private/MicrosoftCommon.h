// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "VideoCommon.h"

#if PLATFORM_MICROSOFT

namespace AVEncoder
{

inline const FString GetComErrorDescription(HRESULT Res)
{
	const uint32 BufSize = 4096;
	WIDECHAR buffer[4096];
	if (::FormatMessageW(FORMAT_MESSAGE_FROM_SYSTEM,
		nullptr,
		Res,
		MAKELANGID(LANG_NEUTRAL, SUBLANG_NEUTRAL),
		buffer,
		sizeof(buffer) / sizeof(*buffer),
		nullptr))
	{
		return buffer;
	}
	else
	{
		return TEXT("[cannot find error description]");
	}
}

// macro to deal with COM calls inside a function that returns `false` on error
#define CHECK_HR(COM_call)\
	{\
		HRESULT Res = COM_call;\
		if (FAILED(Res))\
		{\
			UE_LOG(LogAVEncoder, Error, TEXT("`" #COM_call "` failed: 0x%X - %s"), Res, *GetComErrorDescription(Res));\
			return false;\
		}\
	}

// macro to deal with COM calls inside a function that returns `{}` on error
#define CHECK_HR_DEFAULT(COM_call)\
	{\
		HRESULT Res = COM_call;\
		if (FAILED(Res))\
		{\
			UE_LOG(LogAVEncoder, Error, TEXT("`" #COM_call "` failed: 0x%X - %s"), Res, *GetComErrorDescription(Res));\
			return {};\
		}\
	}

// macro to deal with COM calls inside COM method (that returns HRESULT)
#define CHECK_HR_COM(COM_call)\
	{\
		HRESULT Res = COM_call;\
		if (FAILED(Res))\
		{\
			UE_LOG(LogAVEncoder, Error, TEXT("`" #COM_call "` failed: 0x%X - %s"), Res, *GetComErrorDescription(Res));\
			return Res;\
		}\
	}

// macro to deal with COM calls inside COM method (that simply returns)
#define CHECK_HR_VOID(COM_call)\
	{\
		HRESULT Res = COM_call;\
		if (FAILED(Res))\
		{\
			UE_LOG(LogAVEncoder, Error, TEXT("`" #COM_call "` failed: 0x%X - %s"), Res, *GetComErrorDescription(Res));\
			return;\
		}\
	}

// following commented include causes name clash between UE4 and Windows `IMediaEventSink`,
// we just need a couple of GUIDs from there so the solution is to duplicate them below
//#include "wmcodecdsp.h"

const GUID CLSID_AACMFTEncoder = { 0x93AF0C51, 0x2275, 0x45d2, { 0xA3, 0x5B, 0xF2, 0xBA, 0x21, 0xCA, 0xED, 0x00 } };
const GUID CLSID_MP3ACMCodecWrapper ={ 0x11103421, 0x354c, 0x4cca, { 0xa7, 0xa3, 0x1a, 0xff, 0x9a, 0x5b, 0x67, 0x01 } };
const GUID CLSID_CMSH264EncoderMFT = { 0x6ca50344, 0x051a, 0x4ded, { 0x97, 0x79, 0xa4, 0x33, 0x05, 0x16, 0x5e, 0x35 } };
const GUID CLSID_VideoProcessorMFT = { 0x88753b26, 0x5b24, 0x49bd, { 0xb2, 0xe7, 0xc, 0x44, 0x5c, 0x78, 0xc9, 0x82 } };

// `MF_LOW_LATENCY` is defined in "mfapi.h" for >= WIN8
// UE4 supports lower Windows versions at the moment and so `WINVER` is < `_WIN32_WINNT_WIN8`
// to be able to use `MF_LOW_LATENCY` with default UE4 build we define it ourselves and check actual
// Windows version in runtime
#if (WINVER < _WIN32_WINNT_WIN8)
	const GUID MF_LOW_LATENCY = { 0x9c27891a, 0xed7a, 0x40e1,{ 0x88, 0xe8, 0xb2, 0x27, 0x27, 0xa0, 0x24, 0xee } };
#endif


//#if PLATFORM_WINDOWS
//
//ID3D11Device* GetUE4DxDevice();
//
//#endif

//// scope-disable particular DX11 Debug Layer errors
//#if PLATFORM_WINDOWS
//class FScopeDisabledDxDebugErrors final
//{
//private:
//
//public:
//	FScopeDisabledDxDebugErrors(TArray<D3D11_MESSAGE_ID>&& ErrorsToDisable)
//	{
//		TRefCountPtr<ID3D11Debug> Debug;
//		HRESULT HRes = GetUE4DxDevice()->QueryInterface(__uuidof(ID3D11Debug), reinterpret_cast<void**>(Debug.GetInitReference()));
//
//		if (HRes == E_NOINTERFACE)
//		{
//			// Debug Layer is not enabled, so no need to disable its errors
//			return;
//		}
//
//		if (!SUCCEEDED(HRes) ||
//			!SUCCEEDED(HRes = Debug->QueryInterface(__uuidof(ID3D11InfoQueue), reinterpret_cast<void**>(InfoQueue.GetInitReference()))))
//		{
//			//UE_LOGF(LogAVEncoder, VeryVerbose, "Failed to get ID3D11InfoQueue: 0x%X - %ls", HRes, *_GetComErrorDescription(HRes));
//			return;
//		}
//
//		D3D11_INFO_QUEUE_FILTER filter = {};
//		filter.DenyList.NumIDs = ErrorsToDisable.Num();
//		filter.DenyList.pIDList = ErrorsToDisable.GetData();
//		bSucceeded = SUCCEEDED(InfoQueue->PushStorageFilter(&filter));
//	}
//
//	~FScopeDisabledDxDebugErrors()
//	{
//		if (bSucceeded)
//		{
//			InfoQueue->PopStorageFilter();
//		}
//	}
//
//private:
//	TRefCountPtr<ID3D11InfoQueue> InfoQueue;
//	bool bSucceeded = false;
//};
//#endif


} // namespace AVEncoder


#endif // PLATFORM_MICROSOFT


