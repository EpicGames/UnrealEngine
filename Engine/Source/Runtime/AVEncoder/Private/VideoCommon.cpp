// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoCommon.h"
#include "CodecPacket.h"

namespace AVEncoder
{

#if PLATFORM_WINDOWS
void DebugSetD3D11ObjectName(ID3D11DeviceChild* InD3DObject, const char* InName)
{
	static GUID _D3DDebugObjectName = { 0x429b8c22, 0x9188, 0x4b0c, {0x87, 0x42, 0xac, 0xb0, 0xbf, 0x85, 0xc2, 0x00} };
	if (InD3DObject)
	{
		InD3DObject->SetPrivateData(_D3DDebugObjectName, strlen(InName), InName);
	}
}
#endif

}
