// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoderFactory.h"
#include "CoreMinimal.h"

#if PLATFORM_WINDOWS
// H.264
#include "Decoders/VideoDecoderH264_Impl.h"

// VP8
#include "Decoders/VideoDecoderVP8.h"

// MPEG4
#include "Decoders/VideoDecoderMpeg4.h"

// Dummy test
#include "Decoders/VideoDecoderH264_Dummy.h"
#endif

namespace AVEncoder
{

FThreadSafeCounter			FVideoDecoderFactory::NextID = 1;
bool						FVideoDecoderFactory::bDebugDontRegisterDefaultCodecs = false;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVideoDecoderFactory& FVideoDecoderFactory::Get()
{
	static FVideoDecoderFactory Singleton;
	return Singleton;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

FVideoDecoderFactory::FVideoDecoderFactory()
{
	if (!bDebugDontRegisterDefaultCodecs)
	{
		RegisterDefaultCodecs();
	}
}

FVideoDecoderFactory::~FVideoDecoderFactory()
{
	AvailableDecoders.Empty();
	CreateDecoders.Empty();
}


void FVideoDecoderFactory::Debug_SetDontRegisterDefaultCodecs()
{
	bDebugDontRegisterDefaultCodecs = true;
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FVideoDecoderFactory::Register(const FVideoDecoderInfo& InInfo, const CreateDecoderCallback& InCreateEncoder)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	AvailableDecoders.Push(InInfo);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AvailableDecoders.Last().ID = NextID.Increment();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	CreateDecoders.Push(InCreateEncoder);
}


void FVideoDecoderFactory::RegisterDefaultCodecs()
{
	// Add video decoders supported on this platform to the registry. Decoders that wish to add themselves
	// will call back into our FVideoDecoderFactory::Register() above with capability information and a
	// factory method through which a new instance can be created.
#if PLATFORM_WINDOWS
	FVideoDecoderH264_Impl::Register(*this);
	FVideoDecoderVP8::Register(*this);
	FVideoDecoderMPEG4::Register(*this);
#endif

	// For testing purposes add a dummy decoder last.
#if defined(AVENCODER_VIDEO_DECODER_AVAILABLE_H264_DUMMY)
	FVideoDecoderH264_Dummy::Register(*this);
#endif
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FVideoDecoderFactory::GetInfo(uint32 InID, FVideoDecoderInfo& OutInfo) const
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	for (int32 Index = 0; Index < AvailableDecoders.Num(); ++Index)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (AvailableDecoders[Index].ID == InID)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			OutInfo = AvailableDecoders[Index];
			return true;
		}
	}
	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVideoDecoder* FVideoDecoderFactory::Create(uint32 InID, const FVideoDecoder::FInit& InInit)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoDecoder* Result = nullptr;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	for (int32 Index = 0; Index < AvailableDecoders.Num(); ++Index)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (AvailableDecoders[Index].ID == InID)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			Result = CreateDecoders[Index]();
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (Result && !Result->Setup(InInit))
			{
				Result->Shutdown();
				Result = nullptr;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			break;
		}
	}
	return Result;
}

} /* namespace AVEncoder */
