// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoDecoderInput.h"

namespace AVEncoder
{

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FVideoDecoderInputImpl : public FVideoDecoderInput
{
PRAGMA_ENABLE_DEPRECATION_WARNINGS
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	virtual ~FVideoDecoderInputImpl() = default;
	FVideoDecoderInputImpl(const FInputData& InInputData);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	virtual int32 GetWidth() const override
	{ return Width; }
	virtual int32 GetHeight() const override
	{ return Height; }
	virtual int64 GetPTS() const override
	{ return PTS; }
	virtual const void* GetData() const override
	{ return AccessUnit; }
	virtual int32 GetDataSize() const override
	{ return AccessUnitSize; }
	virtual bool IsKeyframe() const override
	{ return bIsKeyframe; }
	virtual bool IsCompleteFrame() const override
	{ return bIsCompleteFrame; }
	virtual bool HasMissingFrames() const override
	{ return bMissingFrames; }
	virtual int32 GetRotation() const override
	{ return Rotation; }
	virtual int32 GetContentType() const override
	{ return ContentType; }

private:
	const void* AccessUnit;
	int64		PTS;
	int32		AccessUnitSize;
	int32		Width;
	int32		Height;
	int32		Rotation;
	int32		ContentType;
	bool		bIsKeyframe;
	bool		bIsCompleteFrame;
	bool		bMissingFrames;
};

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TSharedPtr<FVideoDecoderInput> FVideoDecoderInput::Create(const FInputData& InInputData)
{
	return MakeShared<FVideoDecoderInputImpl>(InInputData);
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVideoDecoderInputImpl::FVideoDecoderInputImpl(const FInputData& InInputData)
{
	AccessUnit = InInputData.EncodedData;
	AccessUnitSize = InInputData.EncodedDataSize;
	PTS = InInputData.PTS;
	Width = InInputData.Width;
	Height = InInputData.Height;
	Rotation = InInputData.Rotation;
	ContentType = InInputData.ContentType;
	bIsKeyframe = InInputData.bIsKeyframe;
	bIsCompleteFrame = InInputData.bIsComplete;
	bMissingFrames = InInputData.bMissingFrames;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

} /* namespace AVEncoder */
