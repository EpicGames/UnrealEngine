// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Event.h"
#include "HAL/PlatformProcess.h"


namespace AVEncoder
{

namespace DecoderUtilities
{

class FEventSignal
{
public:
	FEventSignal(): Event(FPlatformProcess::GetSynchEventFromPool(true))
	{	}

	// Not copyable!
	FEventSignal(const FEventSignal&) = delete;
	FEventSignal& operator=(const FEventSignal&) = delete;

	~FEventSignal()
	{
		Event->Trigger();
		FPlatformProcess::ReturnSynchEventToPool(Event);
	}

	/**
	 * Waits for the event signal to get set. If already set, this function returns immediately.
	 */
	void Wait()
	{
		Event->Wait();
	}

	/**
	 * Waits for the event signal to get set and clears it immediately. If already set, this function returns immediately.
	 */
	void WaitAndReset()
	{
		Event->Wait();
		Event->Reset();
	}

	/**
	 * Signals the event to be set. If already set, this does nothing.
	 */
	void Signal()
	{
		Event->Trigger();
	}

	/**
	 * Clears the signal. Threads will now wait again until the signal is set once more.
	 */
	void Reset()
	{
		Event->Reset();
	}

	/**
	 * Returns state of the signal.
	 */
	bool IsSignaled() const
	{
		return Event->Wait(0);
	}

	/**
	 * Waits for the event signal to get set within the given time limit.
	 * If already set, this function returns immediately.
	 * Returns true when event was set, false when wait timed out.
	 */
	bool WaitTimeout(int64 InMicroSeconds)
	{
		return Event->Wait(FTimespan::FromMicroseconds((double)InMicroSeconds));
	}

	/**
	 * Waits for the event signal to get set within the given time limit and clears it immediately.
	 * If already set, this function returns immediately.
	 * Returns true when event was set, false when wait timed out.
	 */
	bool WaitTimeoutAndReset(int64 InMicroSeconds)
	{
		check(InMicroSeconds > 0);	// 0 is forbidden. either call Wait() without timeout, or check the signal using IsSignaled()
		if (Event->Wait(FTimespan::FromMicroseconds((double)InMicroSeconds)))
		{
			Event->Reset();
			return true;
		}
		return false;
	}

private:
	mutable FEvent* Event;
};


} /* namespace DecoderUtilities */


} /* namespace AVEncoder */
