// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoEncoder.h"
#include "VideoEncoderCommon.h"

DEFINE_LOG_CATEGORY(LogVideoEncoder);

namespace AVEncoder
{

FVideoEncoder::~FVideoEncoder()
{
	// calling virtual methods from the destructor is bad so 'DestroyLayer' was removed from here
	// its now up to the derived classes to clean up
	for (auto&& layer : Layers)
	{
		if (layer)
		{
			UE_LOGF(LogVideoEncoder, Warning, "Encoder layer may not have been cleaned up!");
			break;
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FVideoEncoder::AddLayer(FLayerConfig const& config)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
    if (GetNumLayers() >= GetMaxLayers())
    {
        UE_LOGF(LogVideoEncoder, Error, "Encoder does not support more than %d layers.", GetMaxLayers());
        return false;
    }

    auto const newLayer = CreateLayer(Layers.Num(), config);
    if (nullptr != newLayer)
    {
        Layers.Push(newLayer);
        return true;
    }
    return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVideoEncoder::FLayerConfig FVideoEncoder::GetLayerConfig(uint32 layerIdx) const
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	if (layerIdx >= GetNumLayers())
	{
		UE_LOGF(LogVideoEncoder, Error, "FVideoEncoder::GetLayerConfig: Layer index %d out of range.", layerIdx);
		return {};
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
    return Layers[layerIdx]->GetConfig();
    PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FVideoEncoder::UpdateLayerConfig(uint32 layerIdx, FLayerConfig const& config)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
    if (layerIdx >= GetNumLayers())
    {
        UE_LOGF(LogVideoEncoder, Error, "FVideoEncoder::UpdateLayerConfig: Layer index %d out of range.", layerIdx);
        return;
    }

    PRAGMA_DISABLE_DEPRECATION_WARNINGS
    Layers[layerIdx]->UpdateConfig(config);
    PRAGMA_ENABLE_DEPRECATION_WARNINGS
}


} /* namespace AVEncoder */
