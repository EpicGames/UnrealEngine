// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoEncoderFactory.h"
#include "VideoEncoderInputImpl.h"
#include "RHI.h"

#if PLATFORM_WINDOWS
#include "ID3D11DynamicRHI.h"
#include "ID3D12DynamicRHI.h"
#endif

#if PLATFORM_DESKTOP || PLATFORM_UNIX
#include "IVulkanDynamicRHI.h"
#endif

#include "Encoders/VideoEncoderH264_Dummy.h"

namespace AVEncoder
{

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FCriticalSection			FVideoEncoderFactory::ProtectSingleton;
FVideoEncoderFactory		FVideoEncoderFactory::Singleton;
FThreadSafeCounter			FVideoEncoderFactory::NextID = 4711;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

PRAGMA_DISABLE_DEPRECATION_WARNINGS
FVideoEncoderFactory& FVideoEncoderFactory::Get()
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	if (!Singleton.bWasSetup)
	{
		ProtectSingleton.Lock();
		if (!Singleton.bWasSetup)
		{
			Singleton.bWasSetup = true;
			if (!Singleton.bDebugDontRegisterDefaultCodecs)
			{
				Singleton.RegisterDefaultCodecs();
			}
		}
		ProtectSingleton.Unlock();
	}
	return Singleton;
}

void FVideoEncoderFactory::Shutdown()
{
	FScopeLock		Guard(&ProtectSingleton);
	if (Singleton.bWasSetup)
	{
		Singleton.bWasSetup = false;
		Singleton.bDebugDontRegisterDefaultCodecs = false;
		Singleton.AvailableEncoders.Empty();
		Singleton.CreateEncoders.Empty();

	// 
#if defined(AVENCODER_VIDEO_ENCODER_AVAILABLE_NVENC)
		FNVENCCommon::Shutdown();
#endif
	}
}

void FVideoEncoderFactory::Debug_SetDontRegisterDefaultCodecs()
{
	check(!Singleton.bWasSetup);
	Singleton.bDebugDontRegisterDefaultCodecs = true;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
void FVideoEncoderFactory::Register(const FVideoEncoderInfo& InInfo, const CreateEncoderCallback& InCreateEncoder)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	AvailableEncoders.Push(InInfo);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	AvailableEncoders.Last().ID = NextID.Increment();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	CreateEncoders.Push(InCreateEncoder);
}

void FVideoEncoderFactory::RegisterDefaultCodecs()
{

#if defined(AVENCODER_VIDEO_ENCODER_AVAILABLE_H264_DUMMY)
	FVideoEncoderH264_Dummy::Register(*this);
#endif

}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FVideoEncoderFactory::GetInfo(uint32 InID, FVideoEncoderInfo& OutInfo) const
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	for (int32 Index = 0; Index < AvailableEncoders.Num(); ++Index)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (AvailableEncoders[Index].ID == InID)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			OutInfo = AvailableEncoders[Index];
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			return true;
		}
	}
	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
bool FVideoEncoderFactory::HasEncoderForCodec(ECodecType CodecType) const
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (const AVEncoder::FVideoEncoderInfo& EncoderInfo : AvailableEncoders)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (EncoderInfo.CodecType == CodecType)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			return true;
		}
	}
	return false;
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
TUniquePtr<FVideoEncoder> FVideoEncoderFactory::Create(uint32 InID, const FVideoEncoder::FLayerConfig& config)
{
	// HACK (M84FIX) create encoder without a ready FVideoEncoderInput
	for (int32 Index = 0; Index < AvailableEncoders.Num(); ++Index)
	{
		if (AvailableEncoders[Index].ID == InID)
		{
			TUniquePtr<FVideoEncoder> Result = CreateEncoders[Index]();
			TSharedPtr<FVideoEncoderInput> Input;

			switch (RHIGetInterfaceType())
			{
		#if PLATFORM_WINDOWS
			case ERHIInterfaceType::D3D11:
				Input = FVideoEncoderInput::CreateForD3D11(GetID3D11DynamicRHI()->RHIGetDevice(), true, IsRHIDeviceAMD()).ToSharedRef();
				break;

			case ERHIInterfaceType::D3D12:
				Input = FVideoEncoderInput::CreateForD3D12(GetID3D12DynamicRHI()->RHIGetDevice_NoMGPU(), true, IsRHIDeviceNVIDIA()).ToSharedRef();
				break;
		#endif // PLATFORM_WINDOWS

		#if PLATFORM_DESKTOP || PLATFORM_UNIX
			case ERHIInterfaceType::Vulkan:
				{
					IVulkanDynamicRHI* VulkanRHI = GetIVulkanDynamicRHI();

					AVEncoder::FVulkanDataStruct VulkanData =
					{
						VulkanRHI->RHIGetVkInstance(),
						VulkanRHI->RHIGetVkPhysicalDevice(),
						VulkanRHI->RHIGetVkDevice()
					};

					Input = FVideoEncoderInput::CreateForVulkan(&VulkanData, true).ToSharedRef();
				}
				break;
		#endif // PLATFORM_DESKTOP || PLATFORM_UNIX

			default:
				continue;
			}

			if (Result && Input && Result->Setup(Input.ToSharedRef(), config))
			{
				return Result;
			}
		}
	}

	return nullptr;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS


PRAGMA_DISABLE_DEPRECATION_WARNINGS
TUniquePtr<FVideoEncoder> FVideoEncoderFactory::Create(uint32 InID, TSharedPtr<FVideoEncoderInput> InInput, const FVideoEncoder::FLayerConfig& config)
PRAGMA_ENABLE_DEPRECATION_WARNINGS
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TUniquePtr<FVideoEncoder>		Result;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	if (InInput)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TSharedRef<FVideoEncoderInput>	Input(InInput.ToSharedRef());
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		for (int32 Index = 0; Index < AvailableEncoders.Num(); ++Index)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (AvailableEncoders[Index].ID == InID)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				Result = CreateEncoders[Index]();
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if (Result && !Result->Setup(MoveTemp(Input), config))
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				{
					Result.Reset();
				}
				break;
			}
		}
	}
	return Result;
}

} /* namespace AVEncoder */
