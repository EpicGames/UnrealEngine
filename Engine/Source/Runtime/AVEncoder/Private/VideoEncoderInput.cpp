// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoEncoderInput.h"
#include "VideoEncoderInputImpl.h"
#include "VideoEncoderCommon.h"
#include "VideoEncoderFactory.h"
#include "AVEncoderDebug.h"
#include "Misc/Paths.h"
#include "GenericPlatform/GenericPlatformMath.h"

#include "Misc/Guid.h"

#if PLATFORM_WINDOWS
#include "MicrosoftCommon.h"
#endif

FString GetGUID()
{
	static FGuid id;
	if (!id.IsValid())
	{
		id = FGuid::NewGuid();
	}

	return id.ToString();
}

namespace AVEncoder
{
	// *** FVideoEncoderInput *************************************************************************

	// --- construct video encoder input based on expected input frame format -------------------------
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<FVideoEncoderInput> FVideoEncoderInput::CreateDummy(bool bIsResizable)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		TSharedPtr<FVideoEncoderInputImpl>	Input = MakeShared<FVideoEncoderInputImpl>();
		Input->bIsResizable = bIsResizable;

		if (!Input->SetupForDummy())
		{
			Input.Reset();
		}
		return Input;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<FVideoEncoderInput> FVideoEncoderInput::CreateForYUV420P(uint32 InWidth, uint32 InHeight, bool bIsResizable)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		TSharedPtr<FVideoEncoderInputImpl>	Input = MakeShared<FVideoEncoderInputImpl>();
		Input->bIsResizable = bIsResizable;

		if (!Input->SetupForYUV420P(InWidth, InHeight))
		{
			Input.Reset();
		}
		return Input;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<FVideoEncoderInput> FVideoEncoderInput::CreateForD3D11(void* InApplicationD3DDevice, bool bIsResizable, bool IsShared)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		TSharedPtr<FVideoEncoderInputImpl>	Input = MakeShared<FVideoEncoderInputImpl>();

#if PLATFORM_WINDOWS
		Input->bIsResizable = bIsResizable;

		if (IsShared)
		{
			if (!Input->SetupForD3D11Shared(static_cast<ID3D11Device*>(InApplicationD3DDevice)))
			{
				Input.Reset();
			}
		}
		else
		{
			if (!Input->SetupForD3D11(static_cast<ID3D11Device*>(InApplicationD3DDevice)))
			{
				Input.Reset();
			}
		}

#endif

		return Input;
	}


	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<FVideoEncoderInput> FVideoEncoderInput::CreateForD3D12(void* InApplicationD3DDevice, bool bIsResizable, bool IsShared)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		TSharedPtr<FVideoEncoderInputImpl>	Input = MakeShared<FVideoEncoderInputImpl>();

#if PLATFORM_WINDOWS
		Input->bIsResizable = bIsResizable;

		if (IsShared)
		{
			if (!Input->SetupForD3D12Shared(static_cast<ID3D12Device*>(InApplicationD3DDevice)))
			{
				Input.Reset();
			}
		}
		else
		{
			if (!Input->SetupForD3D12(static_cast<ID3D12Device*>(InApplicationD3DDevice)))
			{
				Input.Reset();
			}
		}

#endif

		return Input;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<FVideoEncoderInput> FVideoEncoderInput::CreateForCUDA(void* InApplicationContext, bool bIsResizable)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		TSharedPtr<FVideoEncoderInputImpl> Input = MakeShared<FVideoEncoderInputImpl>();
		Input->bIsResizable = bIsResizable;

		if (!Input->SetupForCUDA(reinterpret_cast<CUcontext>(InApplicationContext)))
		{
			Input.Reset();
		}
		return Input;
	}

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<FVideoEncoderInput> FVideoEncoderInput::CreateForVulkan(void* InApplicationVulkanData, bool bIsResizable)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		TSharedPtr<FVideoEncoderInputImpl>	Input = MakeShared<FVideoEncoderInputImpl>();
		Input->bIsResizable = bIsResizable;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FVulkanDataStruct* VulkanData = static_cast<FVulkanDataStruct*>(InApplicationVulkanData);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (!Input->SetupForVulkan(VulkanData->VulkanInstance, VulkanData->VulkanPhysicalDevice, VulkanData->VulkanDevice))
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			Input.Reset();
		}

		return Input;
	}
#endif


	void FVideoEncoderInput::SetMaxNumBuffers(uint32 InMaxNumBuffers)
	{
		MaxNumBuffers = InMaxNumBuffers;
	}

	// --- encoder input frames -----------------------------------------------------------------------



	// *** FVideoEncoderInputImpl *********************************************************************

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderInputImpl::~FVideoEncoderInputImpl()
	{
		{
			FScopeLock Guard(&ProtectFrames);
			if (ActiveFrames.Num() > 0)
			{
				UE_LOGF(LogVideoEncoder, Error, "There are still %d active input frames.", ActiveFrames.Num());
			}

			check(ActiveFrames.Num() == 0);
		}
#if PLATFORM_WINDOWS
		//	DEBUG_D3D11_REPORT_LIVE_DEVICE_OBJECT(FrameInfoD3D.EncoderDeviceD3D11);
#endif
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	bool FVideoEncoderInputImpl::SetupForDummy()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FrameFormat = EVideoFrameFormat::Undefined;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return true;
	}

	bool FVideoEncoderInputImpl::SetupForYUV420P(uint32 InWidth, uint32 InHeight)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FrameFormat = EVideoFrameFormat::YUV420P;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FrameInfoYUV420P.StrideY = InWidth;
		FrameInfoYUV420P.StrideU = (InWidth + 1) / 2;
		FrameInfoYUV420P.StrideV = (InWidth + 1) / 2;

		CollectAvailableEncoders();
		return true;
	}

	bool FVideoEncoderInputImpl::SetupForD3D11(void* InApplicationD3DDevice)
	{
#if PLATFORM_WINDOWS
		TRefCountPtr<IDXGIDevice>	DXGIDevice;
		TRefCountPtr<IDXGIAdapter>	Adapter;

		HRESULT	Result = static_cast<ID3D11Device*>(InApplicationD3DDevice)->QueryInterface(__uuidof(IDXGIDevice), (void**)DXGIDevice.GetInitReference());
		if (Result != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "ID3D11Device::QueryInterface() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}
		else if ((Result = DXGIDevice->GetAdapter(Adapter.GetInitReference())) != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "DXGIDevice::GetAdapter() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}

		uint32				DeviceFlags = 0;
		D3D_FEATURE_LEVEL	FeatureLevel = D3D_FEATURE_LEVEL_11_0;
		D3D_FEATURE_LEVEL	ActualFeatureLevel;

		if ((Result = D3D11CreateDevice(
			Adapter,
			D3D_DRIVER_TYPE_UNKNOWN,
			NULL,
			DeviceFlags,
			&FeatureLevel,
			1,
			D3D11_SDK_VERSION,
			FrameInfoD3D.EncoderDeviceD3D11.GetInitReference(),
			&ActualFeatureLevel,
			FrameInfoD3D.EncoderDeviceContextD3D11.GetInitReference())) != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "D3D11CreateDevice() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}
		DEBUG_SET_D3D11_OBJECT_NAME(FrameInfoD3D.EncoderDeviceD3D11, "FVideoEncoderInputImpl");
		DEBUG_SET_D3D11_OBJECT_NAME(FrameInfoD3D.EncoderDeviceContextD3D11, "FVideoEncoderInputImpl");

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FrameFormat = EVideoFrameFormat::D3D11_R8G8B8A8_UNORM;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		CollectAvailableEncoders();
		return true;

#endif
		return false;
	}

	bool FVideoEncoderInputImpl::SetupForD3D11Shared(void* InApplicationD3DDevice)
	{
#if PLATFORM_WINDOWS

		TRefCountPtr<IDXGIDevice>	DXGIDevice;
		TRefCountPtr<IDXGIAdapter>	Adapter;

		HRESULT		Result = static_cast<ID3D11Device*>(InApplicationD3DDevice)->QueryInterface(__uuidof(IDXGIDevice), (void**)DXGIDevice.GetInitReference());
		if (Result != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "ID3D11Device::QueryInterface() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}
		else if ((Result = DXGIDevice->GetAdapter(Adapter.GetInitReference())) != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "DXGIDevice::GetAdapter() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}

		uint32				DeviceFlags = 0;
		D3D_FEATURE_LEVEL	FeatureLevel = D3D_FEATURE_LEVEL_11_1;
		D3D_FEATURE_LEVEL	ActualFeatureLevel;

		if ((Result = D3D11CreateDevice(
			Adapter,
			D3D_DRIVER_TYPE_UNKNOWN,
			NULL,
			DeviceFlags,
			&FeatureLevel,
			1,
			D3D11_SDK_VERSION,
			FrameInfoD3D.EncoderDeviceD3D11.GetInitReference(),
			&ActualFeatureLevel,
			FrameInfoD3D.EncoderDeviceContextD3D11.GetInitReference())) != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "D3D11CreateDevice() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}
		DEBUG_SET_D3D11_OBJECT_NAME(FrameInfoD3D.EncoderDeviceD3D11, "FVideoEncoderInputImpl");
		DEBUG_SET_D3D11_OBJECT_NAME(FrameInfoD3D.EncoderDeviceContextD3D11, "FVideoEncoderInputImpl");

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FrameFormat = EVideoFrameFormat::D3D11_R8G8B8A8_UNORM;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		CollectAvailableEncoders();
		return true;

#endif
		return false;
	}

	bool FVideoEncoderInputImpl::SetupForD3D12(void* InApplicationD3DDevice)
	{
#if PLATFORM_WINDOWS

		LUID						AdapterLuid = static_cast<ID3D12Device*>(InApplicationD3DDevice)->GetAdapterLuid();
		TRefCountPtr<IDXGIFactory4>	DXGIFactory;
		HRESULT						Result;
		if ((Result = CreateDXGIFactory(IID_PPV_ARGS(DXGIFactory.GetInitReference()))) != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "CreateDXGIFactory() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}
		// get the adapter game uses to render
		TRefCountPtr<IDXGIAdapter>	Adapter;
		if ((Result = DXGIFactory->EnumAdapterByLuid(AdapterLuid, IID_PPV_ARGS(Adapter.GetInitReference()))) != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "DXGIFactory::EnumAdapterByLuid() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}

		uint32				DeviceFlags = 0;
		D3D_FEATURE_LEVEL	FeatureLevel = D3D_FEATURE_LEVEL_12_0; // TODO get this from the adaptor support
		if ((Result = D3D12CreateDevice(Adapter, FeatureLevel, IID_PPV_ARGS(FrameInfoD3D.EncoderDeviceD3D12.GetInitReference()))) != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "D3D11CreateDevice() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FrameFormat = EVideoFrameFormat::D3D12_R8G8B8A8_UNORM;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		CollectAvailableEncoders();
		return true;

#endif

		return false;
	}

	bool FVideoEncoderInputImpl::SetupForD3D12Shared(void* InApplicationD3DDevice)
	{
#if PLATFORM_WINDOWS

		LUID						AdapterLuid = static_cast<ID3D12Device*>(InApplicationD3DDevice)->GetAdapterLuid();
		TRefCountPtr<IDXGIFactory4>	DXGIFactory;
		HRESULT						Result;
		if ((Result = CreateDXGIFactory(IID_PPV_ARGS(DXGIFactory.GetInitReference()))) != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "CreateDXGIFactory() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}
		// get the adapter game uses to render
		TRefCountPtr<IDXGIAdapter>	Adapter;
		if ((Result = DXGIFactory->EnumAdapterByLuid(AdapterLuid, IID_PPV_ARGS(Adapter.GetInitReference()))) != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "DXGIFactory::EnumAdapterByLuid() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}

		uint32				DeviceFlags = 0;
		D3D_FEATURE_LEVEL	FeatureLevel = D3D_FEATURE_LEVEL_11_1;
		D3D_FEATURE_LEVEL	ActualFeatureLevel;
		if ((Result = D3D11CreateDevice(
			Adapter,
			D3D_DRIVER_TYPE_UNKNOWN,
			NULL,
			DeviceFlags,
			&FeatureLevel,
			1,
			D3D11_SDK_VERSION,
			FrameInfoD3D.EncoderDeviceD3D11.GetInitReference(),
			&ActualFeatureLevel,
			FrameInfoD3D.EncoderDeviceContextD3D11.GetInitReference())) != S_OK)
		{
			UE_LOGF(LogVideoEncoder, Error, "D3D11CreateDevice() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			return false;
		}

		if (ActualFeatureLevel != D3D_FEATURE_LEVEL_11_1)
		{
			UE_LOGF(LogVideoEncoder, Error, "D3D11CreateDevice() - failed to create device w/ feature level 11.1 - needed to encode textures from D3D12.");
			FrameInfoD3D.EncoderDeviceD3D11.SafeRelease();
			FrameInfoD3D.EncoderDeviceContextD3D11.SafeRelease();
			return false;
		}

		DEBUG_SET_D3D11_OBJECT_NAME(FrameInfoD3D.EncoderDeviceD3D11, "FVideoEncoderInputImpl");
		DEBUG_SET_D3D11_OBJECT_NAME(FrameInfoD3D.EncoderDeviceContextD3D11, "FVideoEncoderInputImpl");

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FrameFormat = EVideoFrameFormat::D3D11_R8G8B8A8_UNORM;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		CollectAvailableEncoders();
		return true;

#endif

		return false;
	}


	bool FVideoEncoderInputImpl::SetupForCUDA(void* InApplicationContext)
	{
		FrameInfoCUDA.EncoderContextCUDA = static_cast<CUcontext>(InApplicationContext);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FrameFormat = EVideoFrameFormat::CUDA_R8G8B8A8_UNORM;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		CollectAvailableEncoders();
		return true;
	}

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
	bool FVideoEncoderInputImpl::SetupForVulkan(VkInstance InApplicationVulkanInstance, VkPhysicalDevice InApplicationVulkanPhysicalDevice, VkDevice InApplicationVulkanDevice)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FrameInfoVulkan.VulkanInstance = InApplicationVulkanInstance;
		FrameInfoVulkan.VulkanPhysicalDevice = InApplicationVulkanPhysicalDevice;
		FrameInfoVulkan.VulkanDevice = InApplicationVulkanDevice;

		FrameFormat = EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		CollectAvailableEncoders();
		return true;
	}
#endif

	// --- available encoders -------------------------------------------------------------------------

	void FVideoEncoderInputImpl::CollectAvailableEncoders()
	{
		AvailableEncoders.Empty();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		for (const FVideoEncoderInfo& Info : FVideoEncoderFactory::Get().GetAvailable())
		{
			if (Info.SupportedInputFormats.Contains(FrameFormat))
			{
				AvailableEncoders.Push(Info);
			}
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const TArray<FVideoEncoderInfo>& FVideoEncoderInputImpl::GetAvailableEncoders()
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		return AvailableEncoders;
	}

	// --- encoder input frames -----------------------------------------------------------------------

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	bool FVideoEncoderInputImpl::IsUserManagedFrame(const FVideoEncoderInputFrame* InBuffer) const
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		const FVideoEncoderInputFrameImpl* Frame = static_cast<const FVideoEncoderInputFrameImpl*>(InBuffer);
		FScopeLock						Guard(&ProtectFrames);
		for (int32 Index = UserManagedFrames.Num() - 1; Index >= 0; --Index)
		{
			if (UserManagedFrames[Index].Key == Frame)
			{
				return true;
			}
		}
		return false;
	}

	// create a user managed buffer
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderInputFrame* FVideoEncoderInputImpl::CreateBuffer(OnFrameReleasedCallback InOnFrameReleased)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FVideoEncoderInputFrameImpl* Frame = CreateFrame();
		if (Frame)
		{
			FScopeLock						Guard(&ProtectFrames);
			UserManagedFrames.Emplace(Frame, MoveTemp(InOnFrameReleased));
		}
		return Frame;
	}

	// destroy user managed buffer
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderInputImpl::DestroyBuffer(FVideoEncoderInputFrame* InBuffer)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FVideoEncoderInputFrameImpl* Frame = static_cast<FVideoEncoderInputFrameImpl*>(InBuffer);
		FScopeLock						Guard(&ProtectFrames);
		bool							bAnythingRemoved = false;
		for (int32 Index = UserManagedFrames.Num() - 1; Index >= 0; --Index)
		{
			if (UserManagedFrames[Index].Key == Frame)
			{
				UserManagedFrames.RemoveAt(Index);
				bAnythingRemoved = true;
			}
		}
		if (bAnythingRemoved)
		{
			delete Frame;
		}
	}

	// --- encoder input frames -----------------------------------------------------------------------

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	TSharedPtr<FVideoEncoderInputFrame> FVideoEncoderInputImpl::ObtainInputFrame()
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		TSharedPtr<FVideoEncoderInputFrameImpl> Frame;
		FScopeLock						Guard(&ProtectFrames);

		if (!AvailableFrames.IsEmpty())
		{

			AvailableFrames.Dequeue(Frame);

		}
		else
		{
			Frame = MakeShareable(CreateFrame());
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			UE_LOGF(LogVideoEncoder, Verbose, "Created new frame total frames: %d", NumBuffers);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		

		ActiveFrames.Push(Frame);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Frame->SetFrameID(NextFrameID++);

		if (NextFrameID == 0)
		{
			++NextFrameID; // skip 0 id
		}

		Frame->Obtain();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return Frame;
	}

	FVideoEncoderInputFrameImpl* FVideoEncoderInputImpl::CreateFrame()
	{
		FVideoEncoderInputFrameImpl* Frame = new FVideoEncoderInputFrameImpl(this);
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NumBuffers++;
		switch (FrameFormat)
		{
		case EVideoFrameFormat::Undefined:
			UE_LOGF(LogVideoEncoder, Error, "Got undefined frame format!");
			break;
		case EVideoFrameFormat::YUV420P:
			SetupFrameYUV420P(Frame);
			break;
		case EVideoFrameFormat::D3D11_R8G8B8A8_UNORM:
			SetupFrameD3D11(Frame);
			break;
		case EVideoFrameFormat::D3D12_R8G8B8A8_UNORM:
			SetupFrameD3D12(Frame);
			break;
		case EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM:
			SetupFrameVulkan(Frame);
			break;
		case EVideoFrameFormat::CUDA_R8G8B8A8_UNORM:
			SetupFrameCUDA(Frame);
			break;
		default:
			check(false);
			break;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return Frame;
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderInputImpl::ReleaseInputFrame(FVideoEncoderInputFrame* InFrame)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FVideoEncoderInputFrameImpl* InFrameImpl = static_cast<FVideoEncoderInputFrameImpl*>(InFrame);

		FScopeLock Guard(&ProtectFrames);
		// check user managed buffers first
		for (const UserManagedFrame& Frame : UserManagedFrames)
		{
			if (Frame.Key == InFrameImpl)
			{
				Frame.Value(InFrameImpl);
				return;
			}
		}

		TSharedPtr<FVideoEncoderInputFrameImpl>* InFramePtrPtr = ActiveFrames.FindByPredicate([InFrameImpl](TSharedPtr<FVideoEncoderInputFrameImpl> ActiveFrame) { return ActiveFrame.Get() == InFrameImpl; });
		if (!InFramePtrPtr)
		{
			// releasing a non active frame. might be after we flushed or something. ignore it.
			return;
		}

		TSharedPtr<FVideoEncoderInputFrameImpl> InFramePtr = *InFramePtrPtr;
		int32 NumRemoved = ActiveFrames.Remove(InFramePtr);
		check(NumRemoved == 1);
		if (NumRemoved > 0)
		{
			// drop frame if format changed
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (InFrame->GetFormat() != FrameFormat)
			{
				ProtectFrames.Unlock();
				NumBuffers--;
				UE_LOGF(LogVideoEncoder, Verbose, "Deleted buffer (format mismatch) total remaining: %d", NumBuffers);
				return;
			}

			if (!AvailableFrames.IsEmpty() && NumBuffers > MaxNumBuffers)
			{
				ProtectFrames.Unlock();
				NumBuffers--;
				UE_LOGF(LogVideoEncoder, Verbose, "Deleted buffer (too many) total frames: %d", NumBuffers);
				return;
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			AvailableFrames.Enqueue(InFramePtr);
		}
	}

	void FVideoEncoderInputImpl::Flush()
	{
		FScopeLock ScopeLock(&ProtectFrames);
		AvailableFrames.Empty();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		NumBuffers = 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	

	void FVideoEncoderInputImpl::SetupFrameYUV420P(FVideoEncoderInputFrameImpl* Frame)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Frame->SetFormat(EVideoFrameFormat::YUV420P);
		FVideoEncoderInputFrame::FYUV420P& YUV420P = Frame->GetYUV420P();
		YUV420P.StrideY = FrameInfoYUV420P.StrideY;
		YUV420P.StrideU = FrameInfoYUV420P.StrideU;
		YUV420P.StrideV = FrameInfoYUV420P.StrideV;
		YUV420P.Data[0] = YUV420P.Data[1] = YUV420P.Data[2] = nullptr;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	void FVideoEncoderInputImpl::SetupFrameD3D11(FVideoEncoderInputFrameImpl* Frame)
	{
#if PLATFORM_WINDOWS
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Frame->SetFormat(FrameFormat);

		FVideoEncoderInputFrame::FD3D11& Data = Frame->GetD3D11();
		Data.EncoderDevice = FrameInfoD3D.EncoderDeviceD3D11;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}

	void FVideoEncoderInputImpl::SetupFrameD3D12(FVideoEncoderInputFrameImpl* Frame)
	{
#if PLATFORM_WINDOWS
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Frame->SetFormat(FrameFormat);

		if (FrameFormat == EVideoFrameFormat::D3D11_R8G8B8A8_UNORM)
		{
			FVideoEncoderInputFrame::FD3D11& Data = Frame->GetD3D11();
			Data.EncoderDevice = FrameInfoD3D.EncoderDeviceD3D11;
		}
		else
		{
			FVideoEncoderInputFrame::FD3D12& Data = Frame->GetD3D12();
			Data.EncoderDevice = FrameInfoD3D.EncoderDeviceD3D12;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}

	void FVideoEncoderInputImpl::SetupFrameVulkan(FVideoEncoderInputFrameImpl* Frame)
	{
#if PLATFORM_DESKTOP && !PLATFORM_APPLE
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Frame->SetFormat(FrameFormat);

		FVideoEncoderInputFrame::FVulkan& Data = Frame->GetVulkan();
		Data.EncoderDevice = FrameInfoVulkan.VulkanDevice;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}

	void FVideoEncoderInputImpl::SetupFrameCUDA(FVideoEncoderInputFrameImpl* Frame)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Frame->SetFormat(FrameFormat);

		FVideoEncoderInputFrame::FCUDA& Data = Frame->GetCUDA();
		Data.EncoderDevice = FrameInfoCUDA.EncoderContextCUDA;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	// ---

#if PLATFORM_WINDOWS
	TRefCountPtr<ID3D11Device> FVideoEncoderInputImpl::GetD3D11EncoderDevice() const
	{
		return FrameInfoD3D.EncoderDeviceD3D11;
	}

	TRefCountPtr<ID3D12Device> FVideoEncoderInputImpl::GetD3D12EncoderDevice() const
	{
		return FrameInfoD3D.EncoderDeviceD3D12;
	}
#endif

	CUcontext FVideoEncoderInputImpl::GetCUDAEncoderContext() const
	{
		return FrameInfoCUDA.EncoderContextCUDA;
	}

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
	void* FVideoEncoderInputImpl::GetVulkanEncoderDevice() const
	{
		return (void*)&FrameInfoVulkan;
	}
#endif

	// *** FVideoEncoderInputFrame ********************************************************************

	FVideoEncoderInputFrame::FVideoEncoderInputFrame()
		: FrameID(0)
		, TimestampUs(0)
		, TimestampRTP(0)
		, NumReferences(0)
		, Width(0)
		, Height(0)
		, bFreeYUV420PData(false)
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		, Format(EVideoFrameFormat::Undefined)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderInputFrame::FVideoEncoderInputFrame(const FVideoEncoderInputFrame& CloneFrom)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
		: FrameID(CloneFrom.FrameID)
		, TimestampUs(CloneFrom.TimestampUs)
		, TimestampRTP(CloneFrom.TimestampRTP)
		, NumReferences(0)
		, Width(CloneFrom.Width)
		, Height(CloneFrom.Height)
		, bFreeYUV420PData(false)
		, Format(CloneFrom.Format)
	{
#if PLATFORM_WINDOWS
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		D3D11.EncoderDevice = CloneFrom.D3D11.EncoderDevice;
		D3D11.Texture = CloneFrom.D3D11.Texture;
		if ((D3D11.EncoderTexture = CloneFrom.D3D11.EncoderTexture) != nullptr)
		{
			D3D11.EncoderTexture->AddRef();
		}

		D3D12.EncoderDevice = CloneFrom.D3D12.EncoderDevice;
		D3D12.Texture = CloneFrom.D3D12.Texture;
		if ((D3D12.EncoderTexture = CloneFrom.D3D12.EncoderTexture) != nullptr)
		{
			D3D12.EncoderTexture->AddRef();
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		CUDA.EncoderDevice = CloneFrom.CUDA.EncoderDevice;
		CUDA.EncoderTexture = CloneFrom.CUDA.EncoderTexture;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}

	FVideoEncoderInputFrame::~FVideoEncoderInputFrame()
	{
		if (bFreeYUV420PData)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			delete[] YUV420P.Data[0];
			delete[] YUV420P.Data[1];
			delete[] YUV420P.Data[2];
			YUV420P.Data[0] = YUV420P.Data[1] = YUV420P.Data[2] = nullptr;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			bFreeYUV420PData = false;
		}

#if PLATFORM_WINDOWS
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (D3D11.EncoderTexture)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			// check to make sure this frame holds the last reference
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			auto	NumRef = D3D11.EncoderTexture->AddRef();
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			if (NumRef > 2)
			{
				UE_LOGF(LogVideoEncoder, Warning, "VideoEncoderFame - D3D11 input texture still holds %d references.", NumRef);
			}
			// Need to call release twice as we have added an extra reference just above (only way to count how many references we have)
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			D3D11.EncoderTexture->Release();
			D3D11.EncoderTexture->Release();
			D3D11.EncoderTexture = nullptr;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (D3D11.SharedHandle)
		{
			CloseHandle(D3D11.SharedHandle);
			D3D11.SharedHandle = nullptr;
		}
		if (D3D11.Texture && OnReleaseD3D11Texture)
		{
			OnReleaseD3D11Texture(D3D11.Texture);
		}
		if (D3D12.EncoderTexture)
		{
			// check to make sure this frame holds the last reference
			auto	NumRef = D3D12.EncoderTexture->AddRef();
			if (NumRef > 2)
			{
				UE_LOGF(LogVideoEncoder, Warning, "VideoEncoderFame - D3D12 input texture still holds %d references.", NumRef);
			}
			// Need to call release twice as we have added an extra reference just above (only way to count how many references we have)
			D3D12.EncoderTexture->Release();
			D3D12.EncoderTexture->Release();
			D3D12.EncoderTexture = nullptr;
		}
		if (D3D12.Texture && OnReleaseD3D12Texture)
		{
			OnReleaseD3D12Texture(D3D12.Texture);
			D3D12.Texture = nullptr;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// D3D12 specific handle atm
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (CUDA.SharedHandle)
		{
			CloseHandle(CUDA.SharedHandle);
			CUDA.SharedHandle = nullptr;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif 

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (CUDA.EncoderTexture)
		{
			OnReleaseCUDATexture(CUDA.EncoderTexture);
			CUDA.EncoderTexture = nullptr;
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Vulkan.EncoderTexture != VK_NULL_HANDLE)
		{
			OnReleaseVulkanTexture(Vulkan.EncoderTexture);
		}

		if (Vulkan.EncoderSurface)
		{
			OnReleaseVulkanSurface(Vulkan.EncoderSurface);
		}
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
#endif
	}

	void FVideoEncoderInputFrame::AllocateYUV420P()
	{
		if (!bFreeYUV420PData)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			YUV420P.StrideY = Width;
			YUV420P.StrideU = (Width + 1) / 2;
			YUV420P.StrideV = (Width + 1) / 2;;
			YUV420P.Data[0] = new uint8[Height * YUV420P.StrideY];
			YUV420P.Data[1] = new uint8[(Height + 1) / 2 * YUV420P.StrideU];
			YUV420P.Data[2] = new uint8[(Height + 1) / 2 * YUV420P.StrideV];
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			bFreeYUV420PData = true;
		}
	}

	void FVideoEncoderInputFrame::SetYUV420P(const uint8* InDataY, const uint8* InDataU, const uint8* InDataV, uint32 InStrideY, uint32 InStrideU, uint32 InStrideV)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Format == EVideoFrameFormat::YUV420P)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			if (bFreeYUV420PData)
			{
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				delete[] YUV420P.Data[0];
				delete[] YUV420P.Data[1];
				delete[] YUV420P.Data[2];
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				bFreeYUV420PData = false;
			}
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			YUV420P.Data[0] = InDataY;
			YUV420P.Data[1] = InDataU;
			YUV420P.Data[2] = InDataV;
			YUV420P.StrideY = InStrideY;
			YUV420P.StrideU = InStrideU;
			YUV420P.StrideV = InStrideV;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	static FThreadSafeCounter	_VideoEncoderInputFrameCnt{ 0 };

#if PLATFORM_WINDOWS
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderInputFrame::SetTexture(ID3D11Texture2D* InTexture, FReleaseD3D11TextureCallback InOnReleaseTexture)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Format == EVideoFrameFormat::D3D11_R8G8B8A8_UNORM)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			check(D3D11.Texture == nullptr);
			if (!D3D11.Texture)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				TRefCountPtr<IDXGIResource> DXGIResource;
				HANDLE						SharedHandle;
				HRESULT		Result = InTexture->QueryInterface(IID_PPV_ARGS(DXGIResource.GetInitReference()));
				if (FAILED(Result))
				{
					UE_LOGF(LogVideoEncoder, Error, "ID3D11Device::QueryInterface() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
				}
				//
				// NOTE : The HANDLE IDXGIResource::GetSharedHandle gives us is NOT an NT Handle, and therefre we should not call CloseHandle on it
				//
				else if ((Result = DXGIResource->GetSharedHandle(&SharedHandle)) != S_OK)
				{
					UE_LOGF(LogVideoEncoder, Error, "IDXGIResource::GetSharedHandle() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
				}
				else if (SharedHandle == nullptr)
				{
					UE_LOGF(LogVideoEncoder, Error, "IDXGIResource::GetSharedHandle() failed to return a shared texture resource no created as shared? (D3D11_RESOURCE_MISC_SHARED).");
				}
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				else if ((Result = D3D11.EncoderDevice->OpenSharedResource(SharedHandle, __uuidof(ID3D11Texture2D), (LPVOID*)&D3D11.EncoderTexture)) != S_OK)
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				{
					UE_LOGF(LogVideoEncoder, Error, "ID3D11Device::OpenSharedResource() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
				}
				else
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					DEBUG_SET_D3D11_OBJECT_NAME(D3D11.EncoderTexture, "FVideoEncoderInputFrame::SetTexture()");
					D3D11.Texture = InTexture;
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					OnReleaseD3D11Texture = InOnReleaseTexture;
				}
			}
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderInputFrame::SetTexture(ID3D12Resource* InTexture, FReleaseD3D12TextureCallback InOnReleaseTexture)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Format == EVideoFrameFormat::D3D11_R8G8B8A8_UNORM)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			check(D3D12.Texture == nullptr)
			check(D3D11.EncoderTexture == nullptr)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS

			TRefCountPtr<ID3D12Device>	OwnerDevice;
			HRESULT						Result;
			if ((Result = InTexture->GetDevice(IID_PPV_ARGS(OwnerDevice.GetInitReference()))) != S_OK)
			{
				UE_LOGF(LogVideoEncoder, Error, "ID3D11Device::QueryInterface() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			}
			//
			// NOTE: ID3D12Device::CreateSharedHandle gives as an NT Handle, and so we need to call CloseHandle on it
			//
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			else if ((Result = OwnerDevice->CreateSharedHandle(InTexture, NULL, GENERIC_ALL, *FString::Printf(TEXT("%s_FVideoEncoderInputFrame_%d"), *GetGUID(), _VideoEncoderInputFrameCnt.Increment()), &D3D11.SharedHandle)) != S_OK)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				UE_LOGF(LogVideoEncoder, Error, "ID3D12Device::CreateSharedHandle() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
			}
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			else if (D3D11.SharedHandle == nullptr)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				UE_LOGF(LogVideoEncoder, Error, "ID3D12Device::CreateSharedHandle() failed to return a shared texture resource no created as shared? (D3D12_RESOURCE_FLAG_ALLOW_SIMULTANEOUS_ACCESS).");
			}
			else
			{
				TRefCountPtr <ID3D11Device1> Device1;
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				if ((Result = D3D11.EncoderDevice->QueryInterface(IID_PPV_ARGS(Device1.GetInitReference()))) != S_OK)
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				{
					UE_LOGF(LogVideoEncoder, Error, "ID3D11Device::QueryInterface() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
				}
				PRAGMA_DISABLE_DEPRECATION_WARNINGS
				else if ((Result = Device1->OpenSharedResource1(D3D11.SharedHandle, IID_PPV_ARGS(&D3D11.EncoderTexture))) != S_OK)
				PRAGMA_ENABLE_DEPRECATION_WARNINGS
				{
					UE_LOGF(LogVideoEncoder, Error, "ID3D11Device::OpenSharedResource1() failed 0x%X - %ls.", Result, *GetComErrorDescription(Result));
				}
				else
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					DEBUG_SET_D3D11_OBJECT_NAME(D3D11.EncoderTexture, "FVideoEncoderInputFrame::SetTexture()");
					D3D12.Texture = InTexture;
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
					OnReleaseD3D12Texture = InOnReleaseTexture;
				}
			}
		}
		else
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			check(D3D12.Texture == nullptr);
			check(D3D12.EncoderTexture == nullptr);
			D3D12.Texture = InTexture;
			D3D12.EncoderTexture = InTexture;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			OnReleaseD3D12Texture = InOnReleaseTexture;
		}
	}
#endif

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderInputFrame::SetTexture(CUarray InTexture, EUnderlyingRHI UnderlyingRHI, void* SharedHandle, FReleaseCUDATextureCallback InOnReleaseTexture)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Format == EVideoFrameFormat::CUDA_R8G8B8A8_UNORM)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			CUDA.UnderlyingRHI = UnderlyingRHI;
			CUDA.SharedHandle = SharedHandle;
			CUDA.EncoderTexture = InTexture;
			OnReleaseCUDATexture = InOnReleaseTexture;
			if (!CUDA.EncoderTexture)
			{
				UE_LOGF(LogVideoEncoder, Warning, "SetTexture | Cuda device pointer is null");
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}


#if PLATFORM_DESKTOP && !PLATFORM_APPLE
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderInputFrame::SetTexture(VkImage InTexture, FReleaseVulkanTextureCallback InOnReleaseTexture)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Format == EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Vulkan.EncoderTexture = InTexture;
			OnReleaseVulkanTexture = InOnReleaseTexture;
			if (!Vulkan.EncoderTexture)
			{
				UE_LOGF(LogVideoEncoder, Warning, "SetTexture | Vulkan VkImage is null");
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	void FVideoEncoderInputFrame::SetTexture(VkImage InTexture, VkDeviceMemory InTextureDeviceMemory, uint64 InTextureMemorySize, FReleaseVulkanTextureCallback InOnReleaseTexture)
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		if (Format == EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Vulkan.EncoderTexture = InTexture;
			Vulkan.EncoderDeviceMemory = InTextureDeviceMemory;
			Vulkan.EncoderMemorySize = InTextureMemorySize;
			OnReleaseVulkanTexture = InOnReleaseTexture;

			if (!Vulkan.EncoderTexture || !InTextureDeviceMemory)
			{
				UE_LOGF(LogVideoEncoder, Warning, "SetTexture | Vulkan VkImage or VkTextureDeviceMemory is null");
			}
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}
#endif

	// *** FVideoEncoderInputFrameImpl ****************************************************************

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderInputFrameImpl::FVideoEncoderInputFrameImpl(FVideoEncoderInputImpl* InInput)
		: Input(InInput)
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderInputFrameImpl::FVideoEncoderInputFrameImpl(const FVideoEncoderInputFrameImpl& InCloneFrom, FCloneDestroyedCallback InCloneDestroyedCallback)
		: FVideoEncoderInputFrame(InCloneFrom)
		, Input(InCloneFrom.Input)
		, ClonedReference(InCloneFrom.Obtain())
		, OnCloneDestroyed(MoveTemp(InCloneDestroyedCallback))
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FVideoEncoderInputFrameImpl::~FVideoEncoderInputFrameImpl()
	{
		if (ClonedReference)
		{
			ClonedReference->Release();
		}
		else
		{
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	void FVideoEncoderInputFrameImpl::Release() const
	{
		// User managed frames get released without checking reference count, as the reference count doesn't matter for them
		if (Input->IsUserManagedFrame(this))
		{
			Input->ReleaseInputFrame(const_cast<FVideoEncoderInputFrameImpl*>(this));
		}
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		else if (NumReferences.Decrement() == 0)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			if (ClonedReference)
			{
				OnCloneDestroyed(this);
				delete this;
			}
			else
			{
				Input->ReleaseInputFrame(const_cast<FVideoEncoderInputFrameImpl*>(this));
			}
		}
	}

	// Clone frame - this will create a copy that references the original until destroyed
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	const FVideoEncoderInputFrame* FVideoEncoderInputFrameImpl::Clone(FCloneDestroyedCallback InCloneDestroyedCallback) const
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	{
		FVideoEncoderInputFrameImpl* ClonedFrame = new FVideoEncoderInputFrameImpl(*this, MoveTemp(InCloneDestroyedCallback));
		return ClonedFrame;
	}
} /* namespace AVEncoder */
