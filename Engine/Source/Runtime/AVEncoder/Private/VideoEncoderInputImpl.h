// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "VideoEncoderInput.h"
#include "Containers/Queue.h"
#include "Misc/ScopeLock.h"
#include "Templates/RefCounting.h"
#include "CudaModule.h"

// HACK (M84FIX) need to break these dependencies
#if PLATFORM_WINDOWS
struct ID3D11DeviceContext;
#endif

namespace AVEncoder
{
class FVideoEncoderInputFrameImpl;

PRAGMA_DISABLE_DEPRECATION_WARNINGS
class FVideoEncoderInputImpl : public FVideoEncoderInput
{
public:
	FVideoEncoderInputImpl() = default;
	virtual ~FVideoEncoderInputImpl();

	// --- properties

	uint32 GetNumActiveFrames() const { FScopeLock Guard(&ProtectFrames); return ActiveFrames.Num(); }
	bool GetHasFreeFrames() const { FScopeLock Guard(&ProtectFrames); return !AvailableFrames.IsEmpty(); }

	// --- construct video encoder input based on expected input frame format

	bool SetupForDummy();
	bool SetupForYUV420P(uint32 InWidth, uint32 InHeight);

	// set up for an encoder that encodes a D3D11 texture (nvenc)
	bool SetupForD3D11(void* InApplicationD3DDevice);
	
	// set up for an encoder that encodes a D3D11 texture within a feature level 11.1 D3D11 device (amf)
	bool SetupForD3D11Shared(void* InApplicationD3DDevice);
	
	// set up for an encoder that encodes a D3D12 texture (amf)
	bool SetupForD3D12(void* InApplicationD3DDevice);

	// set up for an encoder that encodes a D3D12 texture in the context of a D3D11 device (i.e. nvenc)
	bool SetupForD3D12Shared(void* InApplicationD3DDevice);

	// set up an encoder that encodes a CUArray in a CUDA context
	bool SetupForCUDA(void* InApplicationContext);
	
#if PLATFORM_DESKTOP && !PLATFORM_APPLE
	// set up an encoder that encodes a VkImage in the context of a VkDevice
	bool SetupForVulkan(VkInstance InApplicationVulkanInstance, VkPhysicalDevice InApplicationVulkanPhysicalDevice, VkDevice InApplicationVulkanDevice);
#endif

	// --- available encoders

	// get a list of supported video encoders
	const TArray<FVideoEncoderInfo>& GetAvailableEncoders() override;

	// --- encoder input frames - user managed

	// create a user managed buffer
	FVideoEncoderInputFrame* CreateBuffer(OnFrameReleasedCallback InOnFrameReleased) override;
	// destroy user managed buffer
	void DestroyBuffer(FVideoEncoderInputFrame* Buffer) override;

	// --- encoder input frames - managed by this object

	// obtain a video frame that can be used as a buffer for input to a video encoder
	TSharedPtr<FVideoEncoderInputFrame> ObtainInputFrame() override;

	// release (free) an input frame and make it available for future use
	void ReleaseInputFrame(FVideoEncoderInputFrame* InFrame) override;

	// destroy/release any frames that are not currently in use
	void Flush() override;

	// indicates whether a given frame is a user managed frame or not
	bool IsUserManagedFrame(const FVideoEncoderInputFrame* InBuffer) const;

	// --- input properties

#if PLATFORM_WINDOWS
	TRefCountPtr<ID3D11Device> GetD3D11EncoderDevice() const override;
	TRefCountPtr<ID3D12Device> GetD3D12EncoderDevice() const override;
#endif

	CUcontext GetCUDAEncoderContext() const override;

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
	void* GetVulkanEncoderDevice() const override;
#endif

private:

	// collect any encoder that can handle frame format
	void CollectAvailableEncoders();

	TArray<FVideoEncoderInfo>		AvailableEncoders;

	FVideoEncoderInputFrameImpl* CreateFrame();
	void SetupFrameYUV420P(FVideoEncoderInputFrameImpl* Frame);
	void SetupFrameD3D11(FVideoEncoderInputFrameImpl* Frame);
	void SetupFrameD3D12(FVideoEncoderInputFrameImpl* Frame);
	void SetupFrameVulkan(FVideoEncoderInputFrameImpl* Frame);
	void SetupFrameCUDA(FVideoEncoderInputFrameImpl* Frame);

	struct FFrameInfoDummy
	{
	}								FrameInfoDummy;

	struct FFrameInfoYUV420P
	{
		uint32						StrideY = 0;
		uint32						StrideU = 0;
		uint32						StrideV = 0;
	}								FrameInfoYUV420P;

#if PLATFORM_WINDOWS
	struct FFrameInfoD3D
	{
		TRefCountPtr<ID3D11Device>	EncoderDeviceD3D11;
		TRefCountPtr<ID3D11DeviceContext>	EncoderDeviceContextD3D11;
		TRefCountPtr<ID3D12Device>	EncoderDeviceD3D12;
	}								FrameInfoD3D;
#endif

	struct FFrameInfoCUDA
	{
		CUcontext					EncoderContextCUDA;
	}								FrameInfoCUDA;

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
	FVulkanDataStruct				FrameInfoVulkan;
#endif

	mutable FCriticalSection				ProtectFrames;
	TQueue<TSharedPtr<FVideoEncoderInputFrameImpl>>	AvailableFrames;
	TArray<TSharedPtr<FVideoEncoderInputFrameImpl>>	ActiveFrames;
	using UserManagedFrame = TPair<FVideoEncoderInputFrameImpl*, OnFrameReleasedCallback>;
	TArray<UserManagedFrame>				UserManagedFrames;
};

class FVideoEncoderInputFrameImpl : public FVideoEncoderInputFrame
{
public:
	explicit FVideoEncoderInputFrameImpl(FVideoEncoderInputImpl* InInput);
	FVideoEncoderInputFrameImpl(const FVideoEncoderInputFrameImpl& InCloneFrom) = delete;
	explicit FVideoEncoderInputFrameImpl(const FVideoEncoderInputFrameImpl& InCloneFrom, FCloneDestroyedCallback InCloneDestroyedCallback);
	~FVideoEncoderInputFrameImpl();

	// Clone frame - this will create a copy that references the original until destroyed
	const FVideoEncoderInputFrame* Clone(FCloneDestroyedCallback InCloneDestroyedCallback) const override;

	// Release (decrease reference count) of this input frame
	void Release() const override;

	void SetFormat(EVideoFrameFormat InFormat) { Format = InFormat; }
	void SetWidth(uint32 InWidth) { Width = InWidth; }
	void SetHeight(uint32 InHeight) { Height = InHeight; }

private:
	FVideoEncoderInputImpl*					Input;
	const FVideoEncoderInputFrame*			ClonedReference = nullptr;
	const FCloneDestroyedCallback			OnCloneDestroyed;
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS
} /* namespace AVEncoder */
