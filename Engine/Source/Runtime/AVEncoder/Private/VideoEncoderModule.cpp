// Copyright Epic Games, Inc. All Rights Reserved.

#include "VideoEncoderCommon.h"
#include "Modules/ModuleManager.h"

#include "VideoEncoderFactory.h"

class FAVEncoderModule : public IModuleInterface
{
public:

	void ShutdownModule()
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AVEncoder::FVideoEncoderFactory::Shutdown();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
};

IMPLEMENT_MODULE(FAVEncoderModule, AVEncoder);
