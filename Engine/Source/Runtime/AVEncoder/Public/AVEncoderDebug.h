// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#define DEBUG_SET_D3D11_OBJECT_NAME(D3D11Object, Name) \
{\
	static GUID _D3DDebugObjectNameW = {0x4cca5fd8,0x921f,0x42c8,{0x85,0x66,0x70,0xca,0xf2,0xa9,0xb7,0x41}}; \
	FString		SetName = FString::Printf(TEXT("%s (%s:%d)"), *FString(Name), *FPaths::GetCleanFilename(FString(__FILE__)), __LINE__); \
	D3D11Object->SetPrivateData(_D3DDebugObjectNameW, SetName.Len() * sizeof(TCHAR), *SetName); \
}

#define DEBUG_D3D11_REPORT_LIVE_DEVICE_OBJECT(D3D11Device) \
if((D3D11Device)) \
{ \
	TRefCountPtr<ID3D11Debug>	DebugInterface; \
	if(D3D11Device->QueryInterface(__uuidof(ID3D11Debug), (void**)DebugInterface.GetInitReference()) == S_OK) \
	{ \
		DebugInterface->ReportLiveDeviceObjects(D3D11_RLDO_DETAIL | D3D11_RLDO_IGNORE_INTERNAL); \
	} \
}