// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "MediaPacket.h"
#include "Misc/Timespan.h"
#include "SampleBuffer.h"

namespace AVEncoder
{
	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FAudioFrame
	{
		FTimespan Timestamp;
		FTimespan Duration;
		Audio::TSampleBuffer<float> Data;
	};

	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") IAudioEncoderListener
	{
	public:
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual void OnEncodedAudioFrame(const FMediaPacket& Packet) = 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FAudioEncoder
	{
	public:
		virtual ~FAudioEncoder() {}
		virtual const TCHAR* GetName() const = 0;
		virtual const TCHAR* GetType() const = 0;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual bool Initialize(const FAudioConfig& Config) = 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		/**
		* Shutdown MUST be called before destruction
		*/
		virtual void Shutdown() = 0;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual void Encode(const FAudioFrame& Frame) = 0;
		virtual FAudioConfig GetConfig() const = 0;

		AVENCODER_API virtual void RegisterListener(IAudioEncoderListener& Listener);
		AVENCODER_API virtual void UnregisterListener(IAudioEncoderListener& Listener);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

	protected:
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		AVENCODER_API void OnEncodedAudioFrame(const FMediaPacket& Packet);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	private:

		FCriticalSection ListenersMutex;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TArray<IAudioEncoderListener*> Listeners;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

	};
}
