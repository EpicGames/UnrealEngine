// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/UniquePtr.h"

namespace AVEncoder
{
	class FAudioEncoder;

	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FAudioEncoderFactory
	{
	public:
		virtual ~FAudioEncoderFactory() {}
		virtual const TCHAR* GetName() const = 0;
		virtual TArray<FString> GetSupportedCodecs() const = 0;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual TUniquePtr<FAudioEncoder> CreateEncoder(const FString& Codec) = 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		static AVENCODER_API void RegisterFactory(FAudioEncoderFactory& Factory);
		static AVENCODER_API void UnregisterFactory(FAudioEncoderFactory& Factory);
		static AVENCODER_API FAudioEncoderFactory* FindFactory(const FString& Codec);
		static AVENCODER_API const TArray<FAudioEncoderFactory*> GetAllFactories();

	private:
		static AVENCODER_API TArray<FAudioEncoderFactory*> Factories;
	};
}
