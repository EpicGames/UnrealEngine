// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

// ue
#include "Containers/ArrayView.h"
#include "Misc/Optional.h"
#include "Templates/Function.h"

// eossdk
//#include "eos_platform_prereqs.h"
//#include "eos_base.h"


namespace LibRtc
{
	/** There is the declaration of the single audio sample */
	using FAudioChannelSample = int16_t;

	/** Describes the data format of the audio stream */
	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FAudioStreamFormat
	{
		/** The number of samples per seconds */
		uint32_t SampleRate;
		/** The number of channels in the stream */
		uint32_t NumberChannels;
	};

	/**
	 * Describes the byte buffer of the audio stream
	 *
	 * One sample - one sample byte * NumberBytesPerSample
	 * One frame - one sample * NumberChannels
	 * One second - one frame * SampleRate
	 * The number of samples per seconds - one sample byte * NumberBytesPerSample * NumberChannels * SampleRate
	 */
	using FAudioByteBuffer = TArrayView<const uint8_t>;

	/**
	 * Describes the sample buffer of the audio stream
	 *
	 * One frame - one sample * NumberChannels
	 * One second - one frame * SampleRate
	 * The number of samples per seconds - one sample byte * NumberBytesPerSample * NumberChannels * SampleRate
	 */
	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FAudioSampleBuffer
	{
		/** The sequence of audio samples */
		TArrayView<const FAudioChannelSample> AudioSamples;
		/** The format of audio samples */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FAudioStreamFormat AudioStreamFormat;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FAudioCodecConfig
	{
		int32_t SampleRate;
		int32_t NumChannels;
	};

	/**
	 * The callback is called to encode the sample buffer to the byte-by-byte format.
	 *
	 * The unencoded audio sample buffer is not guaranteed to be available after callback finishes.
	 * The pointer of the encoded audio byte buffer should be valid until the next encode call will happen.
	 *
	 * @param AudioUnencodedSampleBuffer - The sample buffer which is being encoded.
	 * @param [out] OutEncodedAudioByteBuffer - The client side pointer of the byte buffer with encoded information.
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	using OnAudioEncodeCallback = TFunction<void(const FAudioSampleBuffer&, const FAudioByteBuffer& AudioBytes)>;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/**
	 * The callback is called to decode the byte-by-byte buffer to the sample buffer format.
	 *
	 * The undecoded audio byte buffer is not guaranteed to be available after callback finishes.
	 * The pointer of the decoded audio sample buffer should be valid until the next decode call will happen.
	 *
	 * @param AudioUndecodedByteBuffer - The byte buffer which is being decoded.
	 * @param [out] OutDecodedAudioSampleBuffer - The client side pointer of the sample buffer with decoded information.
	 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	using OnAudioDecodeCallback = TFunction<void(const FAudioByteBuffer& AudioBytes, const FAudioSampleBuffer&)>;
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FAudioCodec
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		FAudioCodecConfig AudioCodecConfig;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		OnAudioEncodeCallback OnAudioEncodeCallback;
		OnAudioDecodeCallback OnAudioDecodeCallback;
	};

	/** Information which is used to initialize the incoming audio channel */
	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FIncomingAudioChannelInfo
	{
		// nothing
	};

	/** Information which is used to initialize the outgoing audio channel */
	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FOutgoingAudioChannelInfo
	{
		/** Initial muted state of the audio channel */
		bool bIsMuted;
	};

	enum class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") EAudioMuteReason
	{
		Unsupported,
		Manual,
		NotListening,
		AdminDisabled
	};

	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FAudioMuteInfo
	{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TOptional<EAudioMuteReason> Reason;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
}
