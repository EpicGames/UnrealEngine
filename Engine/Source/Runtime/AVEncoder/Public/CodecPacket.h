// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/Timespan.h"
#include "Templates/SharedPointer.h"

namespace AVEncoder
{
	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FCodecPacket
	{
	public:
		virtual ~FCodecPacket() = default;

		static AVENCODER_API FCodecPacket Create(const uint8* InData, uint32 InDataSize);

		/**
		 * Encoding/Decoding latency
		 */
		struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FTimings
		{
			FTimespan StartTs;
			FTimespan FinishTs;
		};

		TSharedPtr<uint8>	Data;					// pointer to encoded data
		uint32				DataSize = 0;			// number of bytes of encoded data
		bool				IsKeyFrame = false;		// whether or not packet represents a key frame
		uint32				VideoQP = 0;
		uint32 				Framerate;
 		FTimings 			Timings;

	private:
		FCodecPacket() = default;
	};
} /* namespace AVEncoder */
