// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// ue
#include "Misc/Optional.h"
#include "Containers/UnrealString.h"

// librtc
#include "AudioTypes.h"


namespace LibRtc
{
    struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FConfiguration
    {
        struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FData
        {
            static constexpr const TCHAR* kDefaultStunServerUrl = TEXT("stun:stun.l.google.com:19302");

            FString StunServerUrl = kDefaultStunServerUrl;
PRAGMA_DISABLE_DEPRECATION_WARNINGS
            TOptional<FAudioCodec> CustomAudioCodec;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
		};

        static FData& Get()
        {
PRAGMA_DISABLE_DEPRECATION_WARNINGS
            static FData Instance;
            return Instance;
PRAGMA_ENABLE_DEPRECATION_WARNINGS
        }
    };
}
