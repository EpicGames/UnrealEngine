// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "MediaPacket.h"
#include "Templates/RefCounting.h"
#include "VideoCommon.h"

#if PLATFORM_WINDOWS

namespace AVEncoder
{

//
// Wrapper for IMFSample, to make it easier to report errors
//
class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FIMFSampleWrapper
{
public:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FIMFSampleWrapper(EPacketType InMediaType = EPacketType::Invalid, IMFSample* InSample = nullptr)
		: MediaType(InMediaType)
		, Sample(InSample)
	{
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	const IMFSample* GetSample() const
	{
		return Sample;
	}

	IMFSample* GetSample()
	{
		return Sample;
	}

	AVENCODER_API bool CreateSample();

	AVENCODER_API FTimespan GetTime() const;

	AVENCODER_API void SetTime(FTimespan Time);

	AVENCODER_API FTimespan GetDuration() const;

	AVENCODER_API void SetDuration(FTimespan Duration);

	AVENCODER_API void SetIsKeyFrame();

	AVENCODER_API bool IsVideoKeyFrame() const;

	AVENCODER_API int GetBufferCount() const;

	/**
	*
	* Calls the specified function for each buffer the sample contains
	* Signature is "int (int BufferIndex, TArrayView<uint8> Data)
	* The specified function should return true if iteration should continue, or false to finish
	*
	* The return value is what the specified function returned:
	*	true : Iterated through all the buffers
	*	false : Early termination
	*/
	template<typename T>
	bool IterateBuffers(T&& Func)
	{
		int BufferCount = GetBufferCount();
		for (int Idx = 0; Idx < BufferCount; ++Idx)
		{
			TRefCountPtr<IMFMediaBuffer> MediaBuffer = nullptr;
			verify(SUCCEEDED(Sample->GetBufferByIndex(0, MediaBuffer.GetInitReference())));

			BYTE* SrcData = nullptr;
			DWORD MaxLength = 0;
			DWORD CurrentLength = 0;
			verify(SUCCEEDED(MediaBuffer->Lock(&SrcData, &MaxLength, &CurrentLength)));
			bool res = Func(Idx, TArrayView<uint8>(SrcData, CurrentLength));
			verify(SUCCEEDED(MediaBuffer->Unlock()));
			if (!res)
			{
				return false;
			}
		}
		return true;
	}

	bool IsValid() const
	{
		return Sample.IsValid();
	}

	void Reset()
	{
		Sample = nullptr;
	}

	AVENCODER_API FIMFSampleWrapper Clone() const;

private:
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EPacketType MediaType;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	TRefCountPtr<IMFSample> Sample;
};

}

#endif // PLATFORM_WINDOWS

