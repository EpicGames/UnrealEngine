// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace AVEncoder
{
	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoConfig
	{
		FString Codec;
		uint32 Width;
		uint32 Height;
		uint32 Framerate;
		uint32 Bitrate;
	};

	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FAudioConfig
	{
		FString Codec;
		uint32 Samplerate;
		uint32 NumChannels;
		uint32 Bitrate;
	};

	enum class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") EPacketType
	{
		Audio,
		Video,
		Invalid
	};

	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FMediaPacket
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EPacketType Type;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		FTimespan Timestamp;
		FTimespan Duration;
		TArray<uint8> Data;

		union
		{
			struct
			{
				bool bKeyFrame;
				int32 Width;
				int32 Height;
				uint32 FrameAvgQP;
				uint32 Framerate;
			} Video;

			struct
			{
			} Audio;
		};

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		explicit FMediaPacket(EPacketType TypeIn)
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		{
			Type = TypeIn;
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			if (Type == EPacketType::Audio)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
			}
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			else if (Type == EPacketType::Video)
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
			{
				FMemory::Memzero(Video);
			}
		}

		bool IsVideoKeyFrame() const
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			return (Type == EPacketType::Video && Video.bKeyFrame) ? true : false;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	};
}
