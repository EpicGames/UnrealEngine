// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CodecPacket.h"
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"

//
// Windows only include
//
#if (PLATFORM_WINDOWS)

#pragma warning(push)
#pragma warning(disable: 4005)

THIRD_PARTY_INCLUDES_START
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <d3d11.h>
#include <mfobjects.h>
#include <mftransform.h>
#include <mfapi.h>
#include <mferror.h>
#include <mfidl.h>
#include <codecapi.h>
#include <shlwapi.h>
#include <mfreadwrite.h>
#include <d3d11_1.h>
#include <d3d12.h>
#include <dxgi1_4.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_END

#endif // PLATFORM_WINDOWS

#ifndef WMFMEDIA_SUPPORTED_PLATFORM
	#define WMFMEDIA_SUPPORTED_PLATFORM (PLATFORM_WINDOWS && (WINVER >= 0x0600 /*Vista*/) && !UE_SERVER)
#endif

#if PLATFORM_WINDOWS
struct ID3D11DeviceChild;
#endif

namespace AVEncoder
{
	const int64 TimeStampNone = 0x7fffffffll;

	enum class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") EVideoFrameFormat
	{
		Undefined,				// (not-yet) defined format
		YUV420P,				// Planar YUV420 format in CPU memory
		D3D11_R8G8B8A8_UNORM,	//
		D3D12_R8G8B8A8_UNORM,	//
		CUDA_R8G8B8A8_UNORM,
		VULKAN_R8G8B8A8_UNORM,
	};

	enum class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") EH264Profile
	{
		UNKNOWN,
		CONSTRAINED_BASELINE,
		BASELINE,
		MAIN,
		CONSTRAINED_HIGH,
		HIGH,
	};

	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	inline FString ToString(EVideoFrameFormat Format)
	{
		switch (Format)
		{
		case EVideoFrameFormat::YUV420P:
			return FString("EVideoFrameFormat::YUV420P");
		case EVideoFrameFormat::D3D11_R8G8B8A8_UNORM:
			return FString("EVideoFrameFormat::D3D11_R8G8B8A8_UNORM");
		case EVideoFrameFormat::D3D12_R8G8B8A8_UNORM:
			return FString("EVideoFrameFormat::D3D12_R8G8B8A8_UNORM");
		case EVideoFrameFormat::CUDA_R8G8B8A8_UNORM:
			return FString("EVideoFrameFormat::CUDA_R8G8B8A8_UNORM");
		case EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM:
			return FString("EVideoFrameFormat::VULKAN_R8G8B8A8_UNORM");
		case EVideoFrameFormat::Undefined:
		default:
			return FString("EVideoFrameFormat::Undefined");
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	enum class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") ECodecType
	{
		Undefined,
		H264,
		MPEG4,
		VP8,
	};

	// TODO: make enums
	const uint32 H264Profile_ConstrainedBaseline = 1 << 0;
	const uint32 H264Profile_Baseline = 1 << 1;
	const uint32 H264Profile_Main = 1 << 2;
	const uint32 H264Profile_ConstrainedHigh = 1 << 3;
	const uint32 H264Profile_High = 1 << 4;

	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoEncoderInfo
	{
		uint32						ID = 0;
		uint32						MaxWidth = 0;
		uint32						MaxHeight = 0;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ECodecType					CodecType = ECodecType::Undefined;
		TArray<EVideoFrameFormat>	SupportedInputFormats;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		struct
		{
			uint32					SupportedProfiles = 0;
			uint32					MinLevel = 0;
			uint32					MaxLevel = 0;
		}							H264;
	};


	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoDecoderInfo
	{
		uint32						ID = 0;
		uint32						MaxWidth = 0;
		uint32						MaxHeight = 0;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ECodecType					CodecType = ECodecType::Undefined;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

#if PLATFORM_WINDOWS
	void DebugSetD3D11ObjectName(ID3D11DeviceChild* InD3DObject, const char* InName);
#endif
} /* namespace AVEncoder */
