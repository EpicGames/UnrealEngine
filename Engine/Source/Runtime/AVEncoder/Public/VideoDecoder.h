// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "Templates/UnrealTemplate.h"
#include "VideoCommon.h"
#include "VideoDecoderAllocationTypes.h"
#include "VideoDecoderInput.h"

namespace AVEncoder
{
	class FVideoDecoderInput;
	class FVideoDecoderOutput;

	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoDecoder
	{
	public:
		using CreateDecoderAllocationInterfaceCallback = TFunction<void(void* /*InOptions*/, void** /*InOutParamResult*/)>;
		using ReleaseDecoderAllocationInterfaceCallback = TFunction<void(void* /*InOptions*/, void** /*InOutParamResult*/)>;

		struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FInit
		{
			CreateDecoderAllocationInterfaceCallback	CreateDecoderAllocationInterface;
			ReleaseDecoderAllocationInterfaceCallback	ReleaseDecoderAllocationInterface;

			int32		Width;
			int32		Height;
		};

		// --- setup
		virtual bool Setup(const FInit& InInit) = 0;
		// Shuts the decoder down AND destroys it. Do not store a pointer to this decoder in a smart pointer!
		virtual void Shutdown() = 0;

		enum class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") EDecodeResult
		{
			Success,
			Failure
//			TryAgainLater
		};

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual EDecodeResult Decode(const FVideoDecoderInput* InInput) = 0;

		using OnDecodedFrameCallback = TFunction<void(const FVideoDecoderOutput* /*InDecodedFrame*/)>;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		virtual void SetOnDecodedFrame(OnDecodedFrameCallback InCallback) { OnDecodedFrame = MoveTemp(InCallback); }
		virtual void ClearOnDecodedFrame() { OnDecodedFrame = nullptr; }

	protected:
		FVideoDecoder() = default;
		virtual ~FVideoDecoder();

	protected:
		bool CreateDecoderAllocationInterface();
		void ReleaseDecoderAllocationInterface();
		void* GetAllocationInterfaceMethods();
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EFrameBufferAllocReturn AllocateOutputFrameBuffer(FVideoDecoderAllocFrameBufferResult* OutBuffer, const FVideoDecoderAllocFrameBufferParams* InAllocParams);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		struct FPlatformDecoderAllocInterface;

		OnDecodedFrameCallback	OnDecodedFrame;
		CreateDecoderAllocationInterfaceCallback	CreateDecoderAllocationInterfaceFN;
		ReleaseDecoderAllocationInterfaceCallback	ReleaseDecoderAllocationInterfaceFN;
		FPlatformDecoderAllocInterface*				PlatformDecoderAllocInterface = nullptr;
	};
}
