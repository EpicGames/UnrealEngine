// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#pragma pack(push, 8)

/**
 * Type of video frame buffer to allocate.
 */
enum class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") EFrameBufferType
{
	CODEC_RawBuffer = 0,
	CODEC_TextureHandle = 1,
	CODEC_TextureObject = 2
};

/**
 * Video frame buffer allocation parameters.
 */
struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoDecoderAllocFrameBufferParams
{
	/** Type of output frame buffer to allocate */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	EFrameBufferType FrameBufferType;
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	/**
	 * Number of bytes to allocate with the specified alignment.
	 * For a raw buffer the allocation size is required as it may include oversize for padding.
	 * For a texture handle or a handle transfer buffer the allocation size is only what is
	 * required to carry the texture handle, which may be as little as 8 bytes for a single
	 * pointer only.
	 */
	int32_t AllocSize;
	int32_t AllocAlignment;
	int32_t AllocFlags;
	/** Dimensions to allocate raw buffer for. */
	int32_t Width;
	int32_t Height;
	int32_t BytesPerPixel;
	/** Extra custom parameters (platform specific) */
	void** ExtraCustomParameters;
};

/**
 * Return value for the video decoder frame buffer allocation callback.
 */
enum class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") EFrameBufferAllocReturn
{
	/** Buffer successfully allocated. */
	CODEC_Success = 0,
	/** No resources available right now, try again later. */
	CODEC_TryAgainLater = 1,
	/** No resources available or some other failure. */
	CODEC_Failure = 2
};

/**
 * Callback to retain an allocated buffer
 */
DECLARE_DELEGATE_TwoParams(FRetainFrameBuffer, void* /* , This */, void* /*Buffer*/);

/**
 * Callback to release an allocated buffer
 */
DECLARE_DELEGATE_TwoParams(FReleaseFrameBuffer, void* /* , This */, void* /*Buffer*/);

/**
 * Output buffer plane description.
 */
struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FFrameBufferOutPlaneDesc
{
	/** Width of the allocated buffer plane in pixels, including necessary padding. */
	int32_t Width;
	/** Height of the allocated buffer plane in pixels, including necessary padding. */
	int32_t Height;
	/** Bytes per pixel. */
	int32_t BytesPerPixel;
	/** Offset in bytes from the allocated buffer start address to the first pixel of this plane. */
	int32_t ByteOffsetToFirstPixel;
	/** Offset in bytes between two pixels. Multiple of BytesPerPixel. Set larger for interleaved planes (eg. a UV plane). */
	int32_t ByteOffsetBetweenPixels;
	/** Offset in bytes between the same column in the next row. */
	int32_t ByteOffsetBetweenRows;
};

/**
 * Result structure to be filled in by the application.
 */
struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoDecoderAllocFrameBufferResult
{
	/** The buffer the application has allocated. */
	void* AllocatedBuffer;
	/** The actual buffer size the application has allocated. */
	int32_t AllocatedSize;

	/** Information of the allocated buffer planes, if applicable (depending on frame buffer type
	 *  to allocate). Supports at most 4 planes (R,G,B,A). Usually 2 (Y, UV) or 3 (Y,U,V or R,G,B). */
	int32_t AllocatedPlanesNum;
	int32_t AllocatedPlaneLayout;	/* reserved */
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	FFrameBufferOutPlaneDesc AllocatedPlaneDesc[4];
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Callback within the application to retain the allocated buffer. */
	FRetainFrameBuffer RetainCallback;
	/** Callback within the application to release the allocated buffer. */
	FReleaseFrameBuffer ReleaseCallback;
	/** Buffer callback user value (a 'this' pointer to something within the application) for the retain and release callbacks. */
	void* CallbackValue;
};


/**
 * Video frame buffer allocation callback within the application.
 * We call this to get a new output frame buffer to decode into.
 */
PRAGMA_DISABLE_DEPRECATION_WARNINGS
DECLARE_DELEGATE_RetVal_ThreeParams(EFrameBufferAllocReturn, FAllocFrameBuffer, void* /*This*/, const FVideoDecoderAllocFrameBufferParams* /*InAllocParams*/, FVideoDecoderAllocFrameBufferResult* /*OutBuffer*/);
PRAGMA_ENABLE_DEPRECATION_WARNINGS



// ---------- PLATFORM SPECIFIC ----------


DECLARE_DELEGATE_RetVal_ThreeParams(int32_t, FGetD3DDevice, void* /* , This */, void** /* , OutD3DDevice */, int32_t* /* , OutD3DVersion */);

struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoDecoderMethodsWindows
{
	/** Magic cookie to check for if this is indeed the expected structure. */
	uint32_t MagicCookie;	// 'WinX' 0x57696e58
	/** Client 'this' pointer passed in the callbacks. */
	void* This;
	/** Client method to allocate a frame buffer to receive a decoded image. */
	FAllocFrameBuffer AllocateFrameBuffer;

	/** Client method to query the D3D device and D3D version used. */
	FGetD3DDevice GetD3DDevice;
};


DECLARE_DELEGATE_RetVal_FiveParams(int32_t, FMemAlloc, void* /*, This*/, void** /*OutAddress*/, int32_t /*, NumBytes */, int32_t/* , Alignment */, int32_t/* , MemType */);
DECLARE_DELEGATE_RetVal_ThreeParams(int32_t, FMemFree, void* /* , This */, void** /* , InAddress */, int32_t /* , MemType */);
DECLARE_DELEGATE_RetVal_ThreeParams(int32_t, FConfigQuery, void* /* , This */, void** /* , OutInfo */, void** /* , InInfo */);

struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoDecoderMethods
{
	/** Magic cookie to check for if this is indeed the expected structure. */
	uint32_t MagicCookie;
	/** Client 'this' pointer passed in the callbacks. */
	void* This;
	/** Client method to allocate a frame buffer to receive a decoded image. */
	FAllocFrameBuffer AllocateFrameBuffer;

	/** Client method to allocate specific memory. */
	FMemAlloc MemAlloc;
	/** Client method to free specific memory. */
	FMemFree MemFree;
	/** Client method to provide specific decoder configuration settings. */
	FConfigQuery ConfigQuery;
};


#pragma pack(pop)
