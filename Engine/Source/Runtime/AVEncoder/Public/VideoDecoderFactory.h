// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/Platform.h"
#include "Templates/Function.h"
#include "VideoDecoder.h"

#include <HAL/ThreadSafeBool.h>

class FThreadSafeCounter;

namespace AVEncoder
{
	struct FVideoDecoderInfo;

	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoDecoderFactory
	{
	public:
		static AVENCODER_API FVideoDecoderFactory& Get();
		static AVENCODER_API void Debug_SetDontRegisterDefaultCodecs();

		// --- decoder registry

		// the callback type used to create a registered decoder
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		using CreateDecoderCallback = TFunction<FVideoDecoder*()>;

		// register a decoder so that it can be iterated and created
		AVENCODER_API void Register(const FVideoDecoderInfo& InInfo, const CreateDecoderCallback& InCreateDecoder);

		// get a list of registered decoders
		const TArray<FVideoDecoderInfo>& GetAvailable() const { return AvailableDecoders; }
		AVENCODER_API bool GetInfo(uint32 InID, FVideoDecoderInfo& OutInfo) const;

		// --- decoder creation
		// create a decoder instance. If not nullptr it must be destroyed through invoking its Shutdown() method!
		AVENCODER_API FVideoDecoder* Create(uint32 InID, const FVideoDecoder::FInit& InInit);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

	private:
		AVENCODER_API FVideoDecoderFactory();
		AVENCODER_API ~FVideoDecoderFactory();
		FVideoDecoderFactory(const FVideoDecoderFactory&) = delete;
		FVideoDecoderFactory& operator=(const FVideoDecoderFactory&) = delete;

		AVENCODER_API void RegisterDefaultCodecs();

		static AVENCODER_API FThreadSafeCounter		NextID;
		static AVENCODER_API bool						bDebugDontRegisterDefaultCodecs;

		TArray<CreateDecoderCallback>	CreateDecoders;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TArray<FVideoDecoderInfo>		AvailableDecoders;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};

} /* namespace AVEncoder */
