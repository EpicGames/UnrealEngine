// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "VideoCommon.h"
#include "VideoDecoderAllocationTypes.h"

struct FVideoDecoderAllocFrameBufferResult;


namespace AVEncoder
{

	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoDecoderInput
	{
	public:

		struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FInputData
		{
			const void* EncodedData = nullptr;
			int64		PTS = 0;
			int32		EncodedDataSize = 0;
			int32		Width = 0;
			int32		Height = 0;
			int32		Rotation = 0;
			int32		ContentType = 0;
			bool		bIsKeyframe = false;
			bool		bIsComplete = false;
			bool		bMissingFrames = false;
		};

		static AVENCODER_API TSharedPtr<FVideoDecoderInput> Create(const FInputData& InInputData);

		virtual int32 GetWidth() const = 0;
		virtual int32 GetHeight() const = 0;
		virtual int64 GetPTS() const = 0;
		virtual const void* GetData() const = 0;
		virtual int32 GetDataSize() const = 0;
		virtual bool IsKeyframe() const = 0;
		virtual bool IsCompleteFrame() const = 0;
		virtual bool HasMissingFrames() const = 0;
		virtual int32 GetRotation() const = 0;
		virtual int32 GetContentType() const = 0;

	protected:
		FVideoDecoderInput() = default;
		virtual ~FVideoDecoderInput() = default;
		FVideoDecoderInput(const FVideoDecoderInput&) = delete;
		FVideoDecoderInput& operator=(const FVideoDecoderInput&) = delete;
	};



	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoDecoderOutput
	{
	public:
		virtual int32 AddRef() = 0;
		virtual int32 Release() = 0;
		virtual int32 GetWidth() const = 0;
		virtual int32 GetHeight() const = 0;
		virtual int64 GetPTS() const = 0;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual const FVideoDecoderAllocFrameBufferResult* GetAllocatedBuffer() const = 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		virtual int32 GetCropLeft() const = 0;
		virtual int32 GetCropRight() const = 0;
		virtual int32 GetCropTop() const = 0;
		virtual int32 GetCropBottom() const = 0;
		virtual int32 GetAspectX() const = 0;
		virtual int32 GetAspectY() const = 0;
		virtual int32 GetPitchX() const = 0;
		virtual int32 GetPitchY() const = 0;
		virtual uint32 GetColorFormat() const = 0;

	protected:
		virtual ~FVideoDecoderOutput() = default;
		FVideoDecoderOutput() = default;
		FVideoDecoderOutput(const FVideoDecoderOutput&) = delete;
		FVideoDecoderOutput& operator=(const FVideoDecoderOutput&) = delete;
	};



} /* namespace AVEncoder */
