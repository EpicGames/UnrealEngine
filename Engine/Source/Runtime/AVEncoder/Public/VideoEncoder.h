// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "Misc/FrameRate.h"
#include "Misc/ScopeLock.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"
#include "VideoEncoderInput.h"

namespace AVEncoder
{
	class FCodecPacket;
	class FVideoEncoderInput;
	class FVideoEncoderInputFrame;

    class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoEncoder
    {
    public:
        enum class RateControlMode { UNKNOWN, CONSTQP, VBR, CBR };
        enum class MultipassMode { UNKNOWN, DISABLED, QUARTER, FULL };
		enum class H264Profile { AUTO, CONSTRAINED_BASELINE, BASELINE, MAIN, CONSTRAINED_HIGH, HIGH, HIGH444, STEREO, SVC_TEMPORAL_SCALABILITY, PROGRESSIVE_HIGH };

        struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FLayerConfig
        {
            uint32			Width = 0;
            uint32			Height = 0;
            uint32			MaxFramerate = 0;
            int32			MaxBitrate = 0;
            int32			TargetBitrate = 0;
            int32			QPMax = -1;
            int32			QPMin = -1;
            RateControlMode RateControlMode = RateControlMode::CBR;
            MultipassMode	MultipassMode = MultipassMode::FULL;
            bool			FillData = false;
			H264Profile		H264Profile = H264Profile::BASELINE;

			bool operator==(FLayerConfig const& other) const
			{
				return Width == other.Width
					&& Height == other.Height
					&& MaxFramerate == other.MaxFramerate
					&& MaxBitrate == other.MaxBitrate
					&& TargetBitrate == other.TargetBitrate
					&& QPMax == other.QPMax
					&& QPMin == other.QPMin
					&& RateControlMode == other.RateControlMode
					&& MultipassMode == other.MultipassMode
					&& FillData == other.FillData
					&& H264Profile == other.H264Profile;
			}

			bool operator!=(FLayerConfig const& other) const
			{
				return !(*this == other);
			}
        };

        AVENCODER_API virtual ~FVideoEncoder();

        PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual bool Setup(TSharedRef<FVideoEncoderInput> input, FLayerConfig const& config) { return false; }
        PRAGMA_ENABLE_DEPRECATION_WARNINGS
		virtual void Shutdown() {}

        AVENCODER_API virtual bool AddLayer(FLayerConfig const& config);
        uint32 GetNumLayers() const { return static_cast<uint32>(Layers.Num()); }
        virtual uint32 GetMaxLayers() const { return 1; }

		AVENCODER_API FLayerConfig GetLayerConfig(uint32 layerIdx) const;
        AVENCODER_API void UpdateLayerConfig(uint32 layerIdx, FLayerConfig const& config);

        PRAGMA_DISABLE_DEPRECATION_WARNINGS
        using OnFrameEncodedCallback = TFunction<void(const TSharedPtr<FVideoEncoderInputFrame> /* InCompletedFrame */)>;
        using OnEncodedPacketCallback = TFunction<void(uint32 /* LayerIndex */, const TSharedPtr<FVideoEncoderInputFrame> /* Frame */, const FCodecPacket& /* Packet */)>;
        PRAGMA_ENABLE_DEPRECATION_WARNINGS

        struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FEncodeOptions
        {
            bool					bForceKeyFrame = false;
            OnFrameEncodedCallback	OnFrameEncoded;
        };

        void SetOnEncodedPacket(OnEncodedPacketCallback callback) { OnEncodedPacket = MoveTemp(callback); }
        void ClearOnEncodedPacket() { OnEncodedPacket = nullptr; }

        PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual void Encode(const TSharedPtr<FVideoEncoderInputFrame> frame, FEncodeOptions const& options) {}
        PRAGMA_ENABLE_DEPRECATION_WARNINGS

    protected:
        FVideoEncoder() = default;

        class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FLayer
        {
        public:
            explicit FLayer(FLayerConfig const& layerConfig)
                : CurrentConfig(layerConfig)
                , NeedsReconfigure(false)
            {}
            virtual ~FLayer() = default;

			FLayerConfig const& GetConfig() const { return CurrentConfig; }
			void UpdateConfig(FLayerConfig const& config)
			{
				FScopeLock lock(&ConfigMutex);
				CurrentConfig = config;
				NeedsReconfigure = true;
			}

		protected:
			FCriticalSection	ConfigMutex;
            FLayerConfig		CurrentConfig;
            bool				NeedsReconfigure;
        };

		virtual FLayer* CreateLayer(uint32 layerIdx, FLayerConfig const& config) { return nullptr; }
		virtual void DestroyLayer(FLayer* layer) {};

        TArray<FLayer*>			Layers;
        OnEncodedPacketCallback	OnEncodedPacket;
    };
}
