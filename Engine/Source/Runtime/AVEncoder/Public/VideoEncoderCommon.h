// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/ScopeLock.h"
#include "Logging/LogMacros.h"

#include "VideoCommon.h"

DECLARE_LOG_CATEGORY_EXTERN(LogVideoEncoder, Log, All);

namespace AVEncoder
{
} /* namespace AVEncoder */