// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "HAL/Platform.h"
#include "HAL/ThreadSafeBool.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"
#include "Templates/UniquePtr.h"
#include "VideoCommon.h"
#include "VideoEncoder.h"

class FThreadSafeCounter;

namespace AVEncoder
{
	class FVideoEncoderInput;

	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoEncoderFactory
	{
	public:
		static AVENCODER_API FVideoEncoderFactory& Get();
		static AVENCODER_API void Shutdown();
		static AVENCODER_API void Debug_SetDontRegisterDefaultCodecs();

		bool IsSetup() const { return bWasSetup; }

		// --- encoder registry

		// the callback type used to create a registered encoder
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		using CreateEncoderCallback = TFunction<TUniquePtr<FVideoEncoder>()>;

		// register an encoder so that it can be iterated and created
		AVENCODER_API void Register(const FVideoEncoderInfo& InInfo, const CreateEncoderCallback& InCreateEncoder);

		// get a list of registered encoders
		const TArray<FVideoEncoderInfo>& GetAvailable() const { return AvailableEncoders; }
		AVENCODER_API bool GetInfo(uint32 InID, FVideoEncoderInfo& OutInfo) const;
		AVENCODER_API bool HasEncoderForCodec(ECodecType CodecType) const;

		// --- encoder creation

		AVENCODER_API TUniquePtr<FVideoEncoder> Create(uint32 InID, const FVideoEncoder::FLayerConfig& config);
		AVENCODER_API TUniquePtr<FVideoEncoder> Create(uint32 InID, TSharedPtr<FVideoEncoderInput> InInput, const FVideoEncoder::FLayerConfig& config);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	private:
		FVideoEncoderFactory() = default;
		~FVideoEncoderFactory() = default;
		FVideoEncoderFactory(const FVideoEncoderFactory&) = delete;
		FVideoEncoderFactory& operator=(const FVideoEncoderFactory&) = delete;

		AVENCODER_API void RegisterDefaultCodecs();

		static AVENCODER_API FCriticalSection			ProtectSingleton;
		static AVENCODER_API FVideoEncoderFactory		Singleton;
		static AVENCODER_API FThreadSafeCounter		NextID;
		bool							bDebugDontRegisterDefaultCodecs = false;
		FThreadSafeBool					bWasSetup;
		TArray<CreateEncoderCallback>	CreateEncoders;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		TArray<FVideoEncoderInfo>		AvailableEncoders;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	};
    
} /* namespace AVEncoder */
