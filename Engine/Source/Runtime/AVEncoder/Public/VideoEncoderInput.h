// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CudaModule.h"
#include "HAL/Platform.h"
#include "HAL/ThreadSafeCounter.h"
#include "Templates/Function.h"
#include "Templates/RefCounting.h"
#include "Templates/SharedPointer.h"
#include "VideoCommon.h"


#if PLATFORM_DESKTOP && !PLATFORM_APPLE
#include "vulkan/vulkan_core.h"
#endif

#if PLATFORM_WINDOWS
struct ID3D11Device;
struct ID3D11Texture2D;
struct ID3D12Device;
struct ID3D12Resource;
#endif


namespace amf {
	struct AMFVulkanSurface;
}

namespace AVEncoder
{
	class FVideoEncoderInputFrame;

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
	struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVulkanDataStruct
	{
		VkInstance VulkanInstance;
		VkPhysicalDevice VulkanPhysicalDevice;
		VkDevice VulkanDevice;
	};
#endif

	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoEncoderInput
	{
	public:
		// --- construct video encoder input based on expected input frame format
		static AVENCODER_API TSharedPtr<FVideoEncoderInput> CreateDummy(bool isResizable = false);
		static AVENCODER_API TSharedPtr<FVideoEncoderInput> CreateForYUV420P(uint32 InWidth, uint32 InHeight, bool isResizable = false);

		// create input for an encoder that encodes a D3D11 texture 
		static AVENCODER_API TSharedPtr<FVideoEncoderInput> CreateForD3D11(void* InApplicationD3D11Device, bool IsResizable = false, bool IsShared = false);

		// create input for an encoder that encodes a D3D12 texture
		static AVENCODER_API TSharedPtr<FVideoEncoderInput> CreateForD3D12(void* InApplicationD3D12Device, bool IsResizable = false, bool IsShared = false);

		// create input for an encoder that encodes a CUarray
		static AVENCODER_API TSharedPtr<FVideoEncoderInput> CreateForCUDA(void* InApplicationCudaContext, bool IsResizable = false);

		// create input for an encoder that encodes a VkImage
		static AVENCODER_API TSharedPtr<FVideoEncoderInput> CreateForVulkan(void* InApplicationVulkanData, bool IsResizable = false);

		// --- properties
		AVENCODER_API virtual void SetMaxNumBuffers(uint32 InMaxNumBuffers);

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EVideoFrameFormat GetFrameFormat() const { return FrameFormat; }
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// --- available encoders

		// get a list of supported video encoders
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		virtual const TArray<FVideoEncoderInfo>& GetAvailableEncoders() = 0;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		// --- create encoders

		// --- encoder input frames - user managed

		// new packet callback prototype void(uint32 LayerIndex, const FCodecPacket& Packet)
		using OnFrameReleasedCallback = TFunction<void(const FVideoEncoderInputFrame* /* InReleasedFrame */)>;

		// create a user managed buffer
		virtual FVideoEncoderInputFrame* CreateBuffer(OnFrameReleasedCallback InOnFrameReleased) = 0;

		// destroy user managed buffer
		virtual void DestroyBuffer(FVideoEncoderInputFrame* Buffer) = 0;

		// --- encoder input frames - managed by this object

		// obtain a video frame that can be used as a buffer for input to a video encoder
		virtual TSharedPtr<FVideoEncoderInputFrame> ObtainInputFrame() = 0;

		// release (free) an input frame and make it available for future use
		virtual void ReleaseInputFrame(FVideoEncoderInputFrame* InFrame) = 0;

		// destroy/release any frames that are not currently in use
		virtual void Flush() = 0;

	#if PLATFORM_WINDOWS
		virtual TRefCountPtr<ID3D11Device> GetD3D11EncoderDevice() const = 0;
		virtual TRefCountPtr<ID3D12Device> GetD3D12EncoderDevice() const = 0;
	#endif

		virtual CUcontext GetCUDAEncoderContext() const = 0;

	#if PLATFORM_DESKTOP && !PLATFORM_APPLE
		virtual void* GetVulkanEncoderDevice() const = 0;
	#endif

	protected:
		FVideoEncoderInput() = default;
		virtual ~FVideoEncoderInput() = default;
		FVideoEncoderInput(const FVideoEncoderInput&) = delete;
		FVideoEncoderInput& operator=(const FVideoEncoderInput&) = delete;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EVideoFrameFormat				FrameFormat = EVideoFrameFormat::Undefined;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

		uint32 MaxNumBuffers = 3;
		uint32 NumBuffers = 0;

		bool bIsResizable = false;

		uint32 NextFrameID = 0;
	};


	// TODO this should go elsewhere and be made cross platform
	class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVideoEncoderInputFrame
	{
	public:
		// Obtain (increase reference count) of this input frame
		const FVideoEncoderInputFrame* Obtain() const { NumReferences.Increment(); return this; }
		// Release (decrease reference count) of this input frame
		virtual void Release() const = 0;

		// the callback type used to create a registered encoder
		using FCloneDestroyedCallback = TFunction<void(const FVideoEncoderInputFrame* /* InCloneAboutToBeDestroyed */)>;

		// Clone frame - this will create a copy that references the original until destroyed
		virtual const FVideoEncoderInputFrame* Clone(FCloneDestroyedCallback InCloneDestroyedCallback) const = 0;

		void SetFrameID(uint32 id) { FrameID = id; }
		uint32 GetFrameID() const { return FrameID; }

		void SetTimestampUs(int64 timestampUs) { TimestampUs = timestampUs; }
		int64 GetTimestampUs() const { return TimestampUs; }

		void SetTimestampRTP(int64 timestampRTP) { TimestampRTP = timestampRTP; }
		int64 GetTimestampRTP() const { return TimestampRTP; }

		// current format of frame
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EVideoFrameFormat GetFormat() const { return Format; }
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		// width of frame buffer
		void SetWidth(uint32 InWidth) { Width = InWidth; }
		uint32 GetWidth() const { return Width; }
		// height of frame buffer
		void SetHeight(uint32 InHeight) { Height = InHeight; }
		uint32 GetHeight() const { return Height; }

		TFunction<void()> OnTextureEncode;

		// --- YUV420P

		struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FYUV420P
		{
			const uint8*		Data[3] = { nullptr, nullptr, nullptr };
			uint32				StrideY = 0;
			uint32				StrideU = 0;
			uint32				StrideV = 0;
		};

		AVENCODER_API void AllocateYUV420P();
		const FYUV420P& GetYUV420P() const 
		{	
			return YUV420P; 
		}
		
		FYUV420P& GetYUV420P() { return YUV420P; }

		AVENCODER_API void SetYUV420P(const uint8* InDataY, const uint8* InDataU, const uint8* InDataV, uint32 InStrideY, uint32 InStrideU, uint32 InStrideV);

#if PLATFORM_WINDOWS
		// --- D3D11

		struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FD3D11
		{
			ID3D11Texture2D*	Texture = nullptr;
			ID3D11Device*		EncoderDevice = nullptr;
			ID3D11Texture2D*	EncoderTexture = nullptr;
			void*				SharedHandle = nullptr;
		};

		const FD3D11& GetD3D11() const { return D3D11; }
		FD3D11& GetD3D11() { return D3D11; }

		// the callback type used to create a registered encoder
		using FReleaseD3D11TextureCallback = TFunction<void(ID3D11Texture2D*)>;

		AVENCODER_API void SetTexture(ID3D11Texture2D* InTexture, FReleaseD3D11TextureCallback InOnReleaseD3D11Texture);

		// --- D3D12

		struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FD3D12
		{
			ID3D12Resource*		Texture = nullptr;
			ID3D12Device*		EncoderDevice = nullptr;
			ID3D12Resource*		EncoderTexture = nullptr;
		};

		const FD3D12& GetD3D12() const { return D3D12; }
		FD3D12& GetD3D12() { return D3D12; }

		// the callback type used to create a registered encoder
		using FReleaseD3D12TextureCallback = TFunction<void(ID3D12Resource*)>;

		AVENCODER_API void SetTexture(ID3D12Resource* InTexture, FReleaseD3D12TextureCallback InOnReleaseD3D11Texture);

#endif // PLATFORM_WINDOWS

		enum class UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") EUnderlyingRHI
		{
			Undefined,
			D3D11,
			D3D12,
			Vulkan
		};

		// --- CUDA
		struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FCUDA
		{
			CUarray			EncoderTexture = nullptr;
			CUcontext   	EncoderDevice = nullptr;
			void*			SharedHandle = nullptr;

			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			EUnderlyingRHI	UnderlyingRHI = EUnderlyingRHI::Undefined;
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		};

		const FCUDA& GetCUDA() const { return CUDA; }
		FCUDA& GetCUDA() { return CUDA; }

		// the callback type used to create a registered encoder
		using FReleaseCUDATextureCallback = TFunction<void(CUarray)>;

		AVENCODER_API void SetTexture(CUarray InTexture, EUnderlyingRHI UnderlyingRHI, void* SharedHandle, FReleaseCUDATextureCallback InOnReleaseTexture);

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
		// --- Vulkan
		struct UE_DEPRECATED(5.4, "AVEncoder has been deprecated. Please use the AVCodecs plugin family instead.") FVulkan
		{
			VkImage				EncoderTexture = VK_NULL_HANDLE;
			VkDeviceMemory		EncoderDeviceMemory;
			uint64				EncoderMemorySize = 0;
			VkDevice			EncoderDevice;
			mutable void*		EncoderSurface = nullptr;
		};

		const FVulkan& GetVulkan() const { return Vulkan; }
		FVulkan& GetVulkan() { return Vulkan; }

		// the callback type used to create a registered encoder
		using FReleaseVulkanTextureCallback = TFunction<void(VkImage)>;
		using FReleaseVulkanSurfaceCallback = TFunction<void(void*)>;
		mutable FReleaseVulkanSurfaceCallback OnReleaseVulkanSurface;

		AVENCODER_API void SetTexture(VkImage InTexture, FReleaseVulkanTextureCallback InOnReleaseTexture);
		AVENCODER_API void SetTexture(VkImage InTexture, VkDeviceMemory InTextureDeviceMemory, uint64 InTextureSize, FReleaseVulkanTextureCallback InOnReleaseTexture);
#endif

		AVENCODER_API virtual ~FVideoEncoderInputFrame();
	protected:
		AVENCODER_API FVideoEncoderInputFrame();
		AVENCODER_API explicit FVideoEncoderInputFrame(const FVideoEncoderInputFrame& CloneFrom);
		

		uint32									FrameID;
		int64									TimestampUs;
		int64									TimestampRTP;
		mutable FThreadSafeCounter				NumReferences;
		uint32									Width;
		uint32									Height;
		FYUV420P								YUV420P;
		bool									bFreeYUV420PData;

		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		EVideoFrameFormat						Format;
		PRAGMA_ENABLE_DEPRECATION_WARNINGS

#if PLATFORM_WINDOWS
		FD3D11									D3D11;
		FReleaseD3D11TextureCallback			OnReleaseD3D11Texture;
		FD3D12									D3D12;
		FReleaseD3D12TextureCallback			OnReleaseD3D12Texture;
#endif

		FCUDA									CUDA;
		FReleaseCUDATextureCallback				OnReleaseCUDATexture;

#if PLATFORM_DESKTOP && !PLATFORM_APPLE
		FVulkan									Vulkan;
		FReleaseVulkanTextureCallback			OnReleaseVulkanTexture;
#endif
	};


} /* namespace AVEncoder */
